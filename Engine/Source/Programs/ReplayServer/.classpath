<?xml version="1.0" encoding="UTF-8"?>
<classpath>
	<classpathentry kind="src" path="src"/>
	<classpathentry kind="con" path="org.eclipse.jdt.launching.JRE_CONTAINER/org.eclipse.jdt.internal.debug.ui.launcher.StandardVMType/JavaSE-1.8"/>
	<classpathentry kind="lib" path="ThirdParty/Gson/gson-2.3.1.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-alpn-client-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-alpn-server-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-annotations-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-cdi-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-client-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-continuation-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-deploy-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-http-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-io-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-jaas-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-jaspi-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-jmx-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-jndi-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-plus-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-proxy-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-quickstart-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-rewrite-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-schemas-3.1.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-security-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-server-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-servlet-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-servlets-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-util-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-webapp-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/jetty-xml-9.2.7.v20150116.jar"/>
	<classpathentry kind="lib" path="ThirdParty/Jetty/servlet-api-3.1.jar"/>
	<classpathentry kind="lib" path="ThirdParty/MongoDB/mongo-java-driver-2.13.0-sources.jar"/>
	<classpathentry kind="lib" path="ThirdParty/MongoDB/mongo-java-driver-2.13.0.jar" sourcepath="ThirdParty/MongoDB/mongo-java-driver-2.13.0-sources.jar"/>
	<classpathentry kind="output" path="bin"/>
</classpath>
