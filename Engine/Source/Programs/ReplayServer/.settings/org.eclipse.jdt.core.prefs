eclipse.preferences.version=1
org.eclipse.jdt.core.compiler.codegen.inlineJsrBytecode=enabled
org.eclipse.jdt.core.compiler.codegen.targetPlatform=1.8
org.eclipse.jdt.core.compiler.codegen.unusedLocal=preserve
org.eclipse.jdt.core.compiler.compliance=1.8
org.eclipse.jdt.core.compiler.debug.lineNumber=generate
org.eclipse.jdt.core.compiler.debug.localVariable=generate
org.eclipse.jdt.core.compiler.debug.sourceFile=generate
org.eclipse.jdt.core.compiler.problem.assertIdentifier=error
org.eclipse.jdt.core.compiler.problem.enumIdentifier=error
org.eclipse.jdt.core.compiler.source=1.8
