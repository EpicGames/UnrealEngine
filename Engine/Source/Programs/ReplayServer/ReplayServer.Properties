#ReplayServer properties

#General properties

quickTaskFrequencyInSeconds=10
longTaskFrequencyInHours=1

httpPort=80

enableBuiltInWebServer=1

#MongoDB properties

mongoDB_Host=localhost
mongoDB_Port=27017

mongoDB_DBName=replayDB
mongoDB_GridFSBucket=replayFS
mongoDB_ReplayCollection=replayCollection
mongoDB_ViewerCollection=viewerCollection
mongoDB_EventCollection=eventCollection
mongoDB_RecentCollection=recentCollection
mongoDB_LogCollection=logCollection

mongoDB_cleanupStaleLiveSessions_MaxMinutes=1
mongoDB_cleanupOldSessions_MaxDays=30
mongoDB_cleanupStaleViewers_MaxMinutes=3
mongoDB_cleanupLogs_MaxDays=14
mongoDB_cleanupLogs_MaxFinestDays=3
mongoDB_ResetIndexes=0
mongoDB_ConvertReplaysToLower=0