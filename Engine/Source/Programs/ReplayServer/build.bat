@echo off
echo Building replay server...

if not defined UE_JAVA_SDK_PATH (
	echo UE_JAVA_SDK_PATH not defined
	goto Exit
)

if %UE_JAVA_SDK_PATH% == "" (
	echo UE_JAVA_SDK_PATH is blank
	goto Exit
)

echo Using Java SDK path: %UE_JAVA_SDK_PATH%

if not exist %UE_JAVA_SDK_PATH%\javac.exe (
	echo Java SDK appears missing
	goto Exit
)

if not exist %UE_JAVA_SDK_PATH%\jar.exe (
	echo Java SDK appears missing
	goto Exit
)

if exist Intermediate rd Intermediate /s/q
if exist Binaries rd Binaries /s/q

md Intermediate
md Binaries

%UE_JAVA_SDK_PATH%\javac -d Intermediate -sourcepath src\com\epicgames\replayserver\ -cp ThirdParty\Jetty\*;ThirdParty\GSon\*;ThirdParty\MongoDB\* src\com\epicgames\replayserver\*.java
%UE_JAVA_SDK_PATH%\jar cfm Binaries\ReplayServer.jar manifest.txt -C Intermediate com\epicgames\replayserver

xcopy /q ThirdParty\Jetty\jetty-util-9.2.7.v20150116.jar Binaries\ThirdParty\Jetty\
xcopy /q ThirdParty\Jetty\jetty-server-9.2.7.v20150116.jar Binaries\ThirdParty\Jetty\
xcopy /q ThirdParty\Jetty\servlet-api-3.1.jar Binaries\ThirdParty\Jetty\
xcopy /q ThirdParty\Jetty\jetty-security-9.2.7.v20150116.jar Binaries\ThirdParty\Jetty\
xcopy /q ThirdParty\Jetty\jetty-http-9.2.7.v20150116.jar Binaries\ThirdParty\Jetty\
xcopy /q ThirdParty\Jetty\jetty-io-9.2.7.v20150116.jar Binaries\ThirdParty\Jetty\
xcopy /q ThirdParty\Jetty\jetty-servlet-9.2.7.v20150116.jar Binaries\ThirdParty\Jetty\
xcopy /q ThirdParty\MongoDB\mongo-java-driver-2.13.0.jar Binaries\ThirdParty\MongoDB\
xcopy /q ThirdParty\Gson\gson-2.3.1.jar Binaries\ThirdParty\Gson\

xcopy /q ReplayServer.Properties Binaries\

if exist Intermediate rd Intermediate /s/q

:Exit
