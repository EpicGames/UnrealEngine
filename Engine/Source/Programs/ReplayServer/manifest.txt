Main-Class: com.epicgames.replayserver.ReplayServer
Class-Path: 
  ThirdParty\Jetty\jetty-util-9.2.7.v20150116.jar
  ThirdParty\Jetty\jetty-server-9.2.7.v20150116.jar
  ThirdParty\Jetty\servlet-api-3.1.jar
  ThirdParty\Jetty\jetty-security-9.2.7.v20150116.jar
  ThirdParty\Jetty\jetty-http-9.2.7.v20150116.jar
  ThirdParty\Jetty\jetty-io-9.2.7.v20150116.jar
  ThirdParty\Jetty\jetty-servlet-9.2.7.v20150116.jar
  ThirdParty\MongoDB\mongo-java-driver-2.13.0.jar
  ThirdParty\Gson\gson-2.3.1.jar
