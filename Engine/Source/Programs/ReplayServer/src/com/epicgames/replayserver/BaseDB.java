/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.InputStream;
import java.io.OutputStream;
import java.io.PrintWriter;
import java.util.ArrayList;
import java.util.List;
import java.util.logging.Level;

public interface BaseDB 
{
	public static class StreamMetadata 
	{
		public long  metaTime1 = 0;
		public long  metaTime2 = 0;

		StreamMetadata( final long inMetaTime1, final long inMetaTime2 )
		{
			metaTime1 = inMetaTime1;
			metaTime2 = inMetaTime2;
		}
	}

	public static class EventInfo 
	{
		public String 	id		= null;
		public String	group 	= null;
		public String	meta 	= null;
		public long		time1 	= 0;
		public long		time2 	= 0;

		EventInfo( final String inId, final String inGroup, final String inMeta, final long inTime1, final long inTime2 )
		{
			id		= inId;
			group 	= inGroup;
			meta 	= inMeta;
			time1 	= inTime1;
			time2 	= inTime2;
		}
	}

	public class EventList 
	{
		List< EventInfo > events = new ArrayList< EventInfo >();
	}

	public void shutdown();
	public boolean sessionExists( final String sessionName );
	public void createSession( final String appName, final int version, final int changelist, final String sessionName, final String friendlyName, final List< String > userNames, final String metaString ) throws ReplayException;
	public void deleteSession( final String sessionName ) throws ReplayException;
	public void refreshSession( final String 	sessionName, 
								final boolean 	bIsLive, 
								final boolean 	bIsIncomplete, 
								final int 		NumChunks, 
								final int 		DemoTimeInMS, 
								final int 		AddSizeInBytes ) throws ReplayException;
	public void refreshSession( final String sessionName, final int AddSizeInBytes ) throws ReplayException;
	public void writeSessionStream( final String sessionName, final String name, final String metaTime1, final String metaTime2, final InputStream inStream ) throws ReplayException;
	public boolean sessionStreamExists( final String sessionName, final String name );

	public StreamMetadata getStreamMetadata( final String sessionName, final String name ) throws ReplayException;
	public long readSessionStream( final String sessionName, final String name, final OutputStream outStream ) throws ReplayException;

	public void writeEventData( final String 	sessionName, 
								final String 	eventGroup,
								final int		eventTime1,
								final int		eventTime2,
								final String 	eventMeta,
								byte[] 			eventData ) throws ReplayException;
	public byte[] readEventData( final String eventId ) throws ReplayException;
	public List<BaseDB.EventInfo> enumerateEvents( final String sessionName, final String eventGroup ) throws ReplayException;

	public void createViewer( final String sessionName, final String viewerName, final String userName ) throws ReplayException;
	public void refreshViewer( final String sessionName, final String viewerName ) throws ReplayException;
	public void deleteViewer( final String sessionName, final String viewerName ) throws ReplayException;
	public ReplaySessionInfo getSessionInfo( final String sessionName ) throws ReplayException;
	public int getNumSessions( final String appName, final int version, final int changelist );
	public List<ReplaySessionInfo> discoverSessions( final String appName, final int version, final int changelist, final String userString, final String metaString, final int limit );
	public List<ReplaySessionInfo> getRecentSessions( final String appName, final int version, final int changelist, final String userName, final int limit );

	public void log( final Level level, final String str );
	public int printLog( final PrintWriter writer, final Level level, final String subFilter );
	
	public void quickCleanup();
	public void longCleanup();
}
