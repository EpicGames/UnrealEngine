/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.util.TimerTask;

public class CleanupTask 
{
	public static class QuickTask extends TimerTask 
	{
		@Override
		public void run() 
		{		
			ReplayDB.quickCleanup();
		}
	}

	public static class LongTask extends TimerTask 
	{
		@Override
		public void run() 
		{		
			ReplayDB.longCleanup();
		}
	}	
}
