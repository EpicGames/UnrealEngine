/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

public class DeleteSession extends HttpServlet 
{
	private static final long serialVersionUID = 1L;

	public DeleteSession()
	{		
	}

    protected void doGet( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	if ( !request.isUserInRole( "admin" ) )
        {
            response.getWriter().println( "<p>Invalid security role</p><br>" );
            response.getWriter().println( "<A HREF=/>GO BACK</A><br>" );
            return;
        }

		final String sessionName = request.getParameter( "Session" );

        if ( !ReplayDB.sessionExists( sessionName, false, false ) )
    	{
        	ReplayLogger.log( Level.SEVERE, "DeleteSession. Session doesn't exist: " + sessionName );
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    		return;
    	}
        
        try
        {
            ReplayDB.deleteSession( sessionName );
        }
        catch ( Exception e )
        {
        	ReplayLogger.log( Level.SEVERE, "DeleteSession. Exception: " + sessionName );
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    		return;
        }

        response.getWriter().println( "<p>Session Deleted: " + sessionName + "</p>" );
        response.getWriter().println( "<A HREF=/>GO BACK</A><br>" ); 

        ReplayLogger.log( Level.FINE, "DeleteSession. SUCCESS. Path: " + sessionName + ", Addr: " + request.getRemoteAddr() );
    }
}
