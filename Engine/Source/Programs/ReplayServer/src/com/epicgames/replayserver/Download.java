/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.ServletOutputStream;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

public class Download extends HttpServlet
{
	private static final long serialVersionUID = 1L;

	public Download()
	{		
	}

    protected void doGet( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	try 
    	{                  	
    		final long startTime = System.currentTimeMillis();
    		
    		final String sessionName = request.getParameter( "Session" );
    		final String filename = request.getParameter( "Filename" );
    		
    		if ( sessionName == null || filename == null )
    		{
        		ReplayLogger.log( Level.SEVERE, "Download: Missing paramaeters." );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}

    		final String path = sessionName + "/" + filename;
    		ReplaySessionInfo session = ReplayDB.getSessionInfo( sessionName );;
    		
    		final String viewerName = request.getParameter( "Viewer" );
    		
    		if ( viewerName != null )
    		{
        		ReplayDB.refreshViewer( sessionName, viewerName );
    		}

    		if ( session.bIsLive )
            {
        		response.setHeader( "State", "Live" );
            }
            else
            {
                response.setHeader( "State", "Final" );
            }
    		
			response.setHeader( "NumChunks", "" + session.numChunks );
    		response.setHeader( "Time", "" + session.demoTimeInMS );

    		if ( !ReplayDB.sessionStreamExists( sessionName, filename ) )
    		{
    			if ( !session.bIsLive )
    			{    				
    				ReplayLogger.log( Level.WARNING, "Download missing from final stream: " + path );
                    response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    			}
    			return;
    		}

			BaseDB.StreamMetadata metaInfo = ReplayDB.getStreamMetadata( sessionName, filename );

			response.setHeader( "MTime1", "" + metaInfo.metaTime1 );
			response.setHeader( "MTime2", "" + metaInfo.metaTime2 );

			try ( ServletOutputStream outputStream = response.getOutputStream() )
    		{
				final long totalBytes = ReplayDB.readSessionStream( sessionName, filename, outputStream );

    			final long endTime = System.currentTimeMillis();
        		
        		final long totalTime = endTime - startTime;
        		
        		if ( totalTime > 5000 )
        		{
            		ReplayLogger.log( Level.WARNING, "Download: " + path + ", Size: " + totalBytes + " Time: " + totalTime + " ms" );
        		}
    		}
    		catch ( Exception e )
    		{
        		ReplayLogger.log( Level.SEVERE, "Download: Stream failed: " + path );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}        		    		    		
    	} 
    	catch ( Exception e ) 
    	{
    		ReplayLogger.log( Level.SEVERE, "Download. Exception: " + e.getMessage() );
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    }
}
