/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.ServletOutputStream;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

public class DownloadEvent extends HttpServlet
{
	private static final long serialVersionUID = 1L;

    protected void doGet( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	try 
    	{                  	   		
    		final String id = request.getParameter( "ID" );
    		
    		if ( id == null )
    		{
        		ReplayLogger.log( Level.SEVERE, "DownloadEvent: Missing paramaeters." );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}

    		try ( ServletOutputStream outputStream = response.getOutputStream() )
    		{
	    		final long startTime = System.currentTimeMillis();
	
				final byte[] eventData = ReplayDB.readEventData( id );
				
    			final long endTime = System.currentTimeMillis();

	        	outputStream.write( eventData, 0, eventData.length );
        		
        		final long totalTime = endTime - startTime;
        		
        		if ( totalTime > 1000 )
        		{
            		ReplayLogger.log( Level.WARNING, "DownloadEvent: " + id + ", Size: " + eventData.length + " Time: " + totalTime + " ms" );
        		}
        		else
        		{
        			ReplayLogger.log( Level.FINEST, "DownloadEvent: " + id + ", Size: " + eventData.length + " Time: " + totalTime + " ms" );
        		}
    		}
    		catch ( Exception e )
    		{
        		ReplayLogger.log( Level.SEVERE, "DownloadEvent: Stream failed: " + id );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}        		    		    		
    	} 
    	catch ( Exception e ) 
    	{
    		ReplayLogger.log( Level.SEVERE, "DownloadEvent. Exception: " + e.getMessage() );
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    }
}
