/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.util.List;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

import com.google.gson.Gson;
import com.google.gson.GsonBuilder;

public class EnumEvents extends HttpServlet 
{
	private static final long serialVersionUID = 1L;
	
    protected void doGet( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	final String versionName = request.getParameter( "Version" );
    	final String sessionName = request.getParameter( "Session" );
    	final String eventGroup = request.getParameter( "Group" );

    	response.setContentType( "application/json" );

		try
		{
    		final long enumTime1 = System.currentTimeMillis();

    		List<BaseDB.EventInfo> events = ReplayDB.enumerateEvents( sessionName, eventGroup );

    		BaseDB.EventList eventLIst = new BaseDB.EventList();
			eventLIst.events = events;
			
			Gson gson = new GsonBuilder().create();

			String jsonString = gson.toJson( eventLIst );

			response.getWriter().print( jsonString );

			final long enumTime2 = System.currentTimeMillis();
			
			if ( enumTime2 - enumTime1 > 1000 )
			{
	        	ReplayLogger.log( Level.WARNING, "EnumEvents. EnumerateEvents long time: " + ( enumTime2 - enumTime1 ) + ", " +  versionName + "/" + sessionName + "/" + eventGroup );
			}
		}
		catch ( Exception e )
		{
        	ReplayLogger.log( Level.SEVERE, "EnumEvents. EnumerateEvents failed. " + versionName + "/" + sessionName + "/" + eventGroup );
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
            return;
		}
    }
}
