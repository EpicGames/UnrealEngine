/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.util.ArrayList;
import java.util.Date;
import java.util.List;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

import com.google.gson.Gson;
import com.google.gson.GsonBuilder;

public class EnumerateSessions extends HttpServlet 
{
	public class ReplayEnumInfo 
	{
		public String 	appName 		= null;
		public String 	sessionName 	= null;
		public String 	friendlyName 	= null;
		public Date 	timestamp 		= null;
		public int		sizeInBytes 	= 0;
		public int 		demoTimeInMS 	= 0;
		public int		numViewers 		= 0;
		public boolean 	bIsLive 		= false;
		public int		changelist 		= 0;
	}

	public class ReplayEnumList 
	{
		List< ReplayEnumInfo > replays = new ArrayList< ReplayEnumInfo >();
	}

	private static final long serialVersionUID = 1L;
	
    protected void doGet( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	final String appName 		= request.getParameter( "App" );
    	final String versionString 	= request.getParameter( "Version" );
    	final String clString 		= request.getParameter( "CL" );
    	final String recentName 	= request.getParameter( "Recent" );
    	final String userName 		= request.getParameter( "User" );
    	final String metaString 	= request.getParameter( "Meta" );

    	if ( appName == null )
    	{
    		ReplayLogger.log( Level.SEVERE, "EnumerateSessions: appName == null" );
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    		return;
    	}
 
    	final int version = versionString != null ? Integer.parseUnsignedInt( versionString ) : 0;
    	final int changelist = clString != null ? Integer.parseUnsignedInt( clString ) : 0;

    	response.setContentType( "application/json" );

		final int hardLimit = 500;

    	List< ReplaySessionInfo > replaySessions = null;
    	
    	if ( recentName != null )
    	{
    		replaySessions = ReplayDB.getRecentSessions( appName, version, changelist, recentName, hardLimit );
    	}
    	else
    	{
    		replaySessions = ReplayDB.discoverSessions( appName, version, changelist, userName, metaString, hardLimit );
    	}

		Gson gson = new GsonBuilder().setDateFormat( "yyyy-MM-dd'T'HH:mm:ss" ).create();

		ReplayEnumList EnumList = new ReplayEnumList();

		for ( int i = 0; i < replaySessions.size(); i++ )
		{
			ReplaySessionInfo sessionEntry = replaySessions.get( i );

			ReplayEnumInfo replay = new ReplayEnumInfo();

			replay.appName 		= sessionEntry.appName;
			replay.sessionName 	= sessionEntry.sessionName;
			replay.friendlyName = sessionEntry.friendlyName;
			replay.timestamp 	= sessionEntry.createDate;
			replay.sizeInBytes 	= sessionEntry.sizeInBytes;
			replay.demoTimeInMS = sessionEntry.demoTimeInMS;
			replay.numViewers 	= sessionEntry.numViewers;
			replay.bIsLive 		= sessionEntry.bIsLive;
			replay.changelist	= (int)sessionEntry.changelist;

			EnumList.replays.add( replay );
		}
		
		String jsonString = gson.toJson( EnumList );

		response.getWriter().print( jsonString );

        ReplayLogger.log( Level.FINEST, "EnumerateSessions. SUCCESS: " + appName );	
    }
}
