/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.File;
import java.io.InputStream;
import java.io.OutputStream;
import java.io.PrintWriter;
import java.util.ArrayList;
import java.util.Collections;
import java.util.Comparator;
import java.util.Date;
import java.util.List;
import java.util.logging.Level;

import com.google.gson.Gson;


public class FileDB implements BaseDB 
{
	static class ReplayMetaInfo 
	{
		public static final long ExpectedVersion = 1;
		
		public long 	version 		= 0;            
		public boolean 	bIsLive 		= false;
		public boolean 	bIncomplete		= false;
	    public int 		numChunks 		= 0;
	    public int 		demoTimeInMS	= 0;
	    public int 		sizeInBytes		= 0;
		public String 	friendlyName 	= null;
	}

	// Path to the directory of the session
	static String getSessionPath( final String SessionName )
	{
		return SessionName;
	}

	// Name of file the holds the state of the stream (live/final, friendly name, etc)
	static String getMetaFilename( String SessionName )
	{
		return getSessionPath( SessionName ) + "/stream.meta";
	}

	static String getSessionViewerPath( String SessionName )
	{
		return getSessionPath( SessionName ) + "/Viewers";
	}
	
	static String getSessionViewerFilename( String SessionName, String ViewerName )
	{
		return getSessionViewerPath( SessionName ) + "/" + ViewerName;
	}

	static void writeSessionMetaData( String SessionName, String FriendlyName, boolean bIsLive, boolean bIncomplete, int NumChunks, int DemoTimeInMS, int SizeInBytes ) throws ReplayException
	{
		final String TempFilename = getSessionPath( SessionName ) + "/meta.tmp";
		
		ReplayMetaInfo metaInfo = new ReplayMetaInfo();
		
		metaInfo.version		= ReplayMetaInfo.ExpectedVersion;
		metaInfo.bIsLive 		= bIsLive;
		metaInfo.bIncomplete	= bIncomplete; 
		metaInfo.numChunks 		= NumChunks;
		metaInfo.demoTimeInMS 	= DemoTimeInMS;
		metaInfo.sizeInBytes	= SizeInBytes;
		metaInfo.friendlyName 	= FriendlyName;

		Gson gson = new Gson();
		
		final String JsonString = gson.toJson( metaInfo );

		FileUtils.writeStringToFile( TempFilename, JsonString );

		if ( !FileUtils.renameFile( TempFilename, getMetaFilename( SessionName ) ) )
		{
			throw new ReplayException( "WriteSessionMeta. Failed to rename session meta file: " + getMetaFilename( SessionName ) );
		}
	}

	public void shutdown()
	{
	}

	public boolean sessionExists( final String sessionName )
	{
		if ( !FileUtils.directoryExists( getSessionPath( sessionName ) ) )
		{
			return false;
		}
	
		if ( !FileUtils.fileExists( getMetaFilename( sessionName ) ) )
		{
			return false;
		}

		return true;
	}

	public void createSession( final String appName, final int version, final int changelist, final String sessionName, final String friendlyName, final List< String > userNames, final String metaString ) throws ReplayException
	{
		if ( sessionExists( sessionName ) )
		{
			throw new ReplayException( "FileDB.CreateSession: Session already exists." );
		}

		// Create the session directory on the file system
		String SessionPath = getSessionPath( sessionName );

		if ( !FileUtils.directoryExists( SessionPath )  )
		{
			if ( !FileUtils.createDirectory( SessionPath ) )
			{
	        	throw new ReplayException( "FileDB.CreateSession: Create session directory failed: " + SessionPath );
			}
		}
		else
		{
			throw new ReplayException( "FileDB.CreateSession: Session already exists: " + SessionPath );
		}

		writeSessionMetaData( sessionName, friendlyName, true, false, 0, 0, 0 );
	}

	public void deleteSession( final String sessionName ) throws ReplayException
	{
		if ( !sessionExists( sessionName ) )
		{
			throw new ReplayException( "FileDB.DeleteSession: Session doesn't exists." );
		}

		final String sessionPath = getSessionPath( sessionName );
		final String viewerPath = getSessionViewerPath( sessionName );

		// Delete the files in the session directory
		FileUtils.deleteFilesInDirectory( sessionPath );
		
		// Delete the files in the viewer directory
		FileUtils.deleteFilesInDirectory( viewerPath );

		FileUtils.deleteFile( viewerPath );	// Don't warn if it fails, it's not always created

		if ( !FileUtils.deleteFile( sessionPath ) )
		{
			ReplayLogger.log( "DeleteSession: Failed to delete SessionPath: " + sessionPath );
		}
	}

	public void refreshSession( final String 	sessionName, 
								final boolean 	bIsLive, 
								final boolean 	bIsIncomplete, 
								final int 		numChunks, 
								final int 		demoTimeInMS, 
								final int 		sizeInBytes ) throws ReplayException
	{
		ReplaySessionInfo sessionInfo = getSessionInfo( sessionName );

		writeSessionMetaData( sessionName, sessionInfo.friendlyName, bIsLive, bIsIncomplete, numChunks, demoTimeInMS, sizeInBytes );
	}

	public void refreshSession( final String sessionName, final int addSizeInBytes ) throws ReplayException
	{
		throw new ReplayException( "RefreshSession: Not supported." );
	}

	public void writeSessionStream( final String sessionName, final String name, final String metaTime1, final String metaTime2, final InputStream inStream ) throws ReplayException
	{
		final String path = getSessionPath( sessionName );
		
		final String tempFilename = path + "/upload.tmp";

        try ( final FileUtils.ReplayOutputStream outputStream = FileUtils.getReplayOutputStream( tempFilename, false ) )
        {
            byte[] Buffer = new byte[1024*4];

        	int readBytes = 0;

        	while ( ( readBytes = inStream.read( Buffer ) ) != -1 ) 
      		{
      			outputStream.write(  Buffer, 0, readBytes );
      		}           	
        }          
		catch( Exception e )
		{
    		throw new ReplayException( "Upload: Stream failed." );
		}

        if ( !FileUtils.renameFile( tempFilename, path + "/" + name ) )
    	{
    		throw new ReplayException( "Upload: Failed to rename file." );
    	}	
	}

	public boolean sessionStreamExists( final String sessionName, final String name )
	{
		return FileUtils.fileExists( getSessionPath( sessionName ) + "/" + name );
	}

	public StreamMetadata getStreamMetadata( final String sessionName, final String name ) throws ReplayException
	{
		throw new ReplayException( "GetStreamMetadata: Not implemented." );
	}

	public long readSessionStream( final String sessionName, final String name, final OutputStream outStream ) throws ReplayException
	{
		final String filename = getSessionPath( sessionName ) + "/" + name;

		try ( final FileUtils.ReplayInputStream inputStream = FileUtils.getReplayInputStream( filename ) )
		{
			byte[] buffer = new byte[1024 * 4];

        	int readBytes = 0;

        	while ( ( readBytes = inputStream.read( buffer ) ) != -1 ) 
      		{
      			outStream.write( buffer, 0, readBytes );
      		}
    		
    		return inputStream.file.length();
		}
		catch( Exception e )
		{
    		throw new ReplayException( "Upload: Stream failed: " + filename );
		}
	}

	public void writeEventData( final String 	sessionName, 
								final String 	eventGroup,
								final int		eventTime1,
								final int		eventTime2,
								final String 	eventMeta,
								byte[] 			eventData ) throws ReplayException
	{
		throw new ReplayException( "FileDB.WriteEventData: Not implemented." );        
	}

	public byte[] readEventData( final String eventId ) throws ReplayException
	{
		throw new ReplayException( "FileDB.ReadEventData: Not implemented." );        
	}

	public List<BaseDB.EventInfo> enumerateEvents( final String sessionName, final String eventGroup ) throws ReplayException
	{
		throw new ReplayException( "FileDB.EnumerateEvents: Not implemented." );        
	}

	public boolean ViewerExists( final String sessionName, final String viewerName )
	{
		return FileUtils.fileExists( getSessionViewerFilename( sessionName, viewerName ) );
	}

	public void createViewer( final String sessionName, final String viewerName, final String userName ) throws ReplayException
	{
		if ( ViewerExists( sessionName, viewerName ) )
		{
			throw new ReplayException( "FileDB.CreateViewer: Viewer already exists." );        
		}
		
		final String ViewerPath = getSessionViewerPath( sessionName ); 

		try
		{				
			if ( !FileUtils.directoryExists( ViewerPath )  )
			{
				if ( !FileUtils.createDirectory( ViewerPath ) )
				{
		        	throw new ReplayException( "FileDB.CreateViewer: Create viewer directory failed: " + ViewerPath );
				}
			}
		}
		catch ( ReplayException e )
		{
			throw new ReplayException( "FileDB.CreateViewer. Failed to create viewer directory: " + ViewerPath );
		}

		final String viewerFullPath = getSessionViewerFilename( sessionName, viewerName ); 

		if ( !FileUtils.createFile( viewerFullPath, true ) )
		{
    		throw new ReplayException( "FileDB.CreateViewer: Create failed: " + viewerFullPath );
		}
	}

	public void refreshViewer( final String sessionName, final String viewerName ) throws ReplayException
	{
		final String viewerFullPath = getSessionViewerFilename( sessionName, viewerName ); 

		if ( !ViewerExists( sessionName, viewerName ) )
		{
			throw new ReplayException( "FileDB.RefreshViewer: Viewer doesn't exist: " + viewerFullPath );        
		}

		FileUtils.createFile( viewerFullPath, true );
	}

	public void deleteViewer( final String sessionName, final String viewerName ) throws ReplayException
	{
		if ( !ViewerExists( sessionName, viewerName ) )
		{
			throw new ReplayException( "MongoDB.DeleteViewer: Viewer doesn't exist." );
		}

		if ( !FileUtils.deleteFile( getSessionViewerFilename( sessionName, viewerName ) ) )
		{
			throw new ReplayException( "MongoDB.DeleteViewer: Delete failed: " + getSessionViewerFilename( sessionName, viewerName ) );
		}
	}
	
	public ReplaySessionInfo getSessionInfo( final String sessionName ) throws ReplayException
	{
		try
		{
			final String metaFilename = getMetaFilename( sessionName );
			final String jsonString = FileUtils.readStringFromFile( metaFilename );

			Gson gson = new Gson();
			
			ReplayMetaInfo metaInfo = gson.fromJson( jsonString, ReplayMetaInfo.class );
			
			if ( metaInfo.version != ReplayMetaInfo.ExpectedVersion )
			{
				throw new ReplayException( "FileDB.GetSessionInfo: Invalid Meta version: " + metaInfo.version );
			}

			if ( metaInfo.friendlyName == null )
			{
				throw new ReplayException( "FileDB.GetSessionInfo. MetaInfo.FriendlyName == null" );
			}
			
			return new ReplaySessionInfo( 	"fixme",
										0,
										0,
										sessionName, 
										metaInfo.friendlyName, 
										metaInfo.bIsLive,
										metaInfo.bIncomplete,
										metaInfo.numChunks, 
										metaInfo.demoTimeInMS, 
										new Date( FileUtils.getFileCreationDate( metaFilename ) ), 
										new Date( FileUtils.getFileLastModified( metaFilename ) ), 
										metaInfo.sizeInBytes, 
										0 );			
		}
		catch ( Exception e )
		{
			throw new ReplayException( "FileDB.GetSessionInfo: There was an error: " + e.getMessage() );
		}		
	}

	public int getNumSessions( final String appName, final int version, final int changelist )
	{
		return 0;
	}

	public List<ReplaySessionInfo> discoverSessions( final String appName, final int version, final int changelist, final String userString, final String metaString, final int limit )
	{
		List<ReplaySessionInfo> sessions = new ArrayList<ReplaySessionInfo>();
		
		File[] sessionFiles = FileUtils.listFiles( "" );
		
		if ( sessionFiles == null )
		{
			return sessions;
		}
		
		// Search sessions within each version
		for ( final File sessionEntry : sessionFiles )
		{
	        if ( !sessionEntry.isDirectory() )
	        {
	        	continue;
	        }

	        try 
    		{
	        	sessions.add( getSessionInfo( sessionEntry.getName() ) );
    		}
    		catch ( Exception e )
    		{
    	        final long sessionAgeInSeconds = ( System.currentTimeMillis() - sessionEntry.lastModified() ) / 1000;

    	        if ( sessionAgeInSeconds > 5 )
    			{
    	        	// Don't worry about sessions that are really new, give the files a chance to settle down before assuming their invalid
    				ReplayLogger.log( "FileDB.DiscoverSessions. Invalid session: " + sessionEntry.getName() );
        	        try
        	        {
        				deleteSession( sessionEntry.getName() );
        	        }
        	        catch ( Exception e2 )
        	        {
        				ReplayLogger.log( "FileDB.DiscoverSessions: Delete session failed " + sessionEntry.getName() );
        	        }
    			}
    		}
		}

		// Sort newest to oldest
		Collections.sort( sessions, new Comparator< ReplaySessionInfo >()
		{
		    public int compare( ReplaySessionInfo s1, ReplaySessionInfo s2 ) 
		    {
		        if ( s1.bIsLive && !s2.bIsLive )
		        {
		        	return -1;
		        }
		        else if ( !s1.bIsLive && s2.bIsLive )
		        {
		        	return 1;
		        }
		    	
		        return s2.createDate.compareTo( s1.createDate );
		    }
		});

        return sessions;
	}

	public List<ReplaySessionInfo> getRecentSessions( final String appName, final int version, final int changelist, final String userName, final int limit )
	{
		return new ArrayList<ReplaySessionInfo>();	
	}

	public void log( final Level level, final String str )
	{
	}
	
	public int printLog( final PrintWriter writer, final Level level, final String subFilter )
	{
		return 0;
	}

	public void quickCleanup()
	{
		// Loop over all of the sessions and clean up old ones
		for ( final ReplaySessionInfo sessionEntry : discoverSessions( null, 0, 0, null, null, 0 ) )
		{		
			if ( sessionEntry.bIsLive && sessionEntry.GetRefreshTimeInSeconds() >= 60 )
			{
				ReplayLogger.log( "FileDB.Cleanup: Marking incomplete live session: " + sessionEntry.sessionName );

				try
				{
					ReplaySessionInfo session = getSessionInfo( sessionEntry.sessionName );

					refreshSession( sessionEntry.sessionName, false, true, sessionEntry.numChunks, sessionEntry.demoTimeInMS, session.sizeInBytes );
				}
				catch ( Exception e )
				{
					ReplayLogger.log( "FileDB.Cleanup: Failed to refresh, deleting stale live session: " + sessionEntry.sessionName );

					try
					{
						deleteSession( sessionEntry.sessionName );
					}
					catch ( Exception e2 )
					{
						ReplayLogger.log( "FileDB.Cleanup: Failed to delete session: " + sessionEntry.sessionName );
					}
				}
			}
			/*
			else if ( sessionEntry.Viewers != null )
			{
				// If this session is fine, make sure any viewers are up to date
				for ( final ReplayViewer viewerEntry : sessionEntry.Viewers )
				{					
					if ( viewerEntry.GetRefreshTimeInSeconds() >= 60 )
					{
						ReplayLogger.Log( "FileDB.Cleanup: Deleting stale viewer: " + sessionEntry.SessionName + "/" + viewerEntry.ViewerName );

						try
						{
							ReplayDB.DeleteViewer( viewerEntry.SessionName, viewerEntry.ViewerName );
						}
						catch ( Exception e2 )
						{
							ReplayLogger.Log( "FileDB.Cleanup: Failed to delete viewer: " + sessionEntry.SessionName + "/" + viewerEntry.ViewerName );
						}
					}
				}
			}
			*/
		}
	}

	public void longCleanup()
	{
	}
}
