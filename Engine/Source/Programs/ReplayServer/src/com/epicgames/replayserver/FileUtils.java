/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.BufferedReader;
import java.io.BufferedWriter;
import java.io.File;
import java.io.FileInputStream;
import java.io.FileNotFoundException;
import java.io.FileOutputStream;
import java.io.FileReader;
import java.io.FileWriter;
import java.io.IOException;
import java.nio.file.Files;
import java.nio.file.Path;
import java.nio.file.Paths;
import java.nio.file.StandardCopyOption;
import java.nio.file.attribute.BasicFileAttributes;

public class FileUtils 
{
	static class ReplayInputStream extends FileInputStream 
	{
		File file = null;
	
		ReplayInputStream( File InFile ) throws FileNotFoundException
		{
			super( InFile );
	
			file = InFile;
		}
	}

	static class ReplayOutputStream  extends FileOutputStream 
	{
		File file = null;

		ReplayOutputStream( File InFile ) throws FileNotFoundException
		{
			super( InFile );

			file = InFile;
		}

		ReplayOutputStream( File InFile, boolean bAppend ) throws FileNotFoundException
		{
			super( InFile, bAppend );

			file = InFile;
		}
	}

	// Hard coded for testing
	static String rootReplayFolder = "d:/ReplayFiles/";

	static boolean directoryExists( String DirPath )
	{
		final File testDir = new File( rootReplayFolder + DirPath );
	
		return ( testDir.exists() && testDir.isDirectory() );
	}

	static boolean fileExists( String DirPath )
	{
		final File testDir = new File( rootReplayFolder + DirPath );
	
		return ( testDir.exists() && testDir.isFile() );
	}

	static long fileLastModified( String FilePath )
	{
		final File openFile = new File( rootReplayFolder + FilePath );
	
		if ( !openFile.exists() || !openFile.isFile() )
		{
			return 0;
		}
	
		return openFile.lastModified();
	}

	static boolean createDirectory( String DirPath )
	{	
		final String fullDirPath = rootReplayFolder + DirPath;
		
		final File dirFile = new File( fullDirPath );
		
		return dirFile.mkdirs();
	}
	
	static boolean createFile( String FilePath, boolean bReplace )
	{
        // Hard coded for testing
		final File createFile = new File( rootReplayFolder + FilePath );
		
		if ( createFile.exists() )
		{
			if ( !bReplace )
			{
				return false;
			}
			createFile.delete();
		}

		try 
		{
			if ( !createFile.createNewFile() )
			{
				return false;
			}
		}
		catch ( IOException e )
		{
			return false;
		}
		
		return true;
	}
	
	static boolean deleteFile( String FilePath )
	{
		final File createFile = new File( rootReplayFolder + FilePath );
		
		if ( !createFile.exists() )
		{
			return false;
		}

		return createFile.delete();
	}

	static boolean deleteDirectory( String DirPath )
	{
		return deleteFile( DirPath );
	}

	static File[] listFiles( String DirPath )
	{
        final File fileFolder = new File( rootReplayFolder + DirPath );
        
        if ( !fileFolder.isDirectory() )
        {
        	return null;
        }

		return fileFolder.listFiles();
	}
	
	static boolean deleteFilesInDirectory( String Path )
	{
		boolean result = true;

		File[] files = FileUtils.listFiles( Path );
		
		if ( files == null )
		{
			return true;
		}

		for ( final File fileEntry : files )
		{
			if ( fileEntry.isFile() )
			{
				if ( !fileEntry.delete() )
				{
					ReplayLogger.log( "Failed to delete file: " + fileEntry.getAbsolutePath() );
					result = false;
				}
			}
		}
		
		return result;
	}
	
	static boolean renameFile( final String Source, final String Dest )
	{
        try 
        {
            final File sourceFile = new File( rootReplayFolder + Source );
            final File destFile = new File( rootReplayFolder + Dest );
            
        	Files.move( sourceFile.toPath(), destFile.toPath(), 
            		StandardCopyOption.ATOMIC_MOVE,
                    StandardCopyOption.REPLACE_EXISTING//,
                    //StandardCopyOption.COPY_ATTRIBUTES
            );
        } 
        catch ( Exception e ) 
        {
            ReplayLogger.log( "Rename file failed. Source: " + Source + " Dest: " + Dest + " : " + e.getMessage() );
            return false;
        }

        return true;
	}
	
	static ReplayInputStream getReplayInputStream( String FilePath ) throws FileNotFoundException
	{
		final File openFile = new File( FileUtils.rootReplayFolder + FilePath );
	
		if ( !openFile.exists() || !openFile.isFile() || !openFile.canRead() )
		{
			throw new FileNotFoundException();
		}
	
		return new ReplayInputStream( openFile );
	}
	
	static ReplayOutputStream getReplayOutputStream( String FilePath, boolean bAppend ) throws IOException
	{
		final File createFile = new File( FileUtils.rootReplayFolder + FilePath );
		
		if ( !bAppend && createFile.exists() )
		{
			createFile.delete();
		}
		
		return new ReplayOutputStream( createFile, bAppend );
	}
	
	static void writeStringToFile( final String Filename, final String Str ) throws ReplayException
	{
		try ( final BufferedWriter writer = new BufferedWriter( new FileWriter( rootReplayFolder + Filename ) ) )
		{
			writer.write( Str );
		}
		catch( Exception e )
		{
			throw new ReplayException( "WriteStringToFile: Failed to write: " + Filename );
		}
	}

	static String readStringFromFile( final String Filename ) throws ReplayException
	{
		try ( final BufferedReader reader = new BufferedReader( new FileReader( rootReplayFolder + Filename ) ) )
		{
			return reader.readLine();
		}
		catch( Exception e )
		{
			throw new ReplayException( "ReadStringFromFile: Failed to read: " + Filename );
		}
	}
	
	static long getFileCreationDate( final String Filename )
	{
		try
		{
			final Path filePath = Paths.get( rootReplayFolder + Filename );
			final BasicFileAttributes attr = Files.readAttributes( filePath, BasicFileAttributes.class );

			return attr.creationTime().toMillis();
		}
		catch ( Exception e )
		{
			return 0;
		}
	}

	static long getFileLastModified( final String Filename )
	{
		return new File( rootReplayFolder + Filename ).lastModified();
	}
}
