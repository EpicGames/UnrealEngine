/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.text.DateFormat;
import java.text.SimpleDateFormat;
import java.util.List;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

public class Index extends HttpServlet 
{
	private static final long serialVersionUID = 1L;

	public Index()
	{		
	}

    protected void doGet( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
		try 
		{
			final long startTime = System.currentTimeMillis();

	        if ( request.getSession().isNew() )
	        {
	            ReplayLogger.log( Level.INFO, "New connection: " + request.getRemoteAddr() + ":" + request.getRemotePort() );
	        }

	        final String refresh = request.getParameter( "Refresh" );

    		response.setContentType( "text/html" );
	    	response.setStatus( HttpServletResponse.SC_OK );
	    	
	        response.getWriter().println( "<h1> Replay Sessions </h1>" );

	        if ( refresh != null )
	    	{
		    	response.getWriter().println( "<meta http-equiv=\"refresh\" content=\"5\">" );
		        response.getWriter().println( "<A HREF=/>SHOW ALL</A><br>" ); 
	    	}
	    	else
	    	{
		        response.getWriter().println( "<A HREF=/?Refresh=yes>REFRESH LAST 10</A><br>" ); 
	    	}

	        response.getWriter().println( "<A HREF=/viewfile/>VIEW LOG</A>, " ); 
	        response.getWriter().println( "<A HREF=/viewfile/?level=FINEST>FINEST</A>, " ); 
	        response.getWriter().println( "<A HREF=/viewfile/?level=FINER>FINER</A>, " ); 
	        response.getWriter().println( "<A HREF=/viewfile/?level=FINE>FINE</A>, " ); 
	        response.getWriter().println( "<A HREF=/viewfile/?level=INFO>INFO</A>, " ); 
	        response.getWriter().println( "<A HREF=/viewfile/?level=WARNING>WARNING</A>, " ); 
	        response.getWriter().println( "<A HREF=/viewfile/?level=SEVERE>SEVERE</A>, " ); 
	        response.getWriter().println( "<A HREF=/viewfile/?filter=StartUploading>StartUploading</A>, " ); 
	        response.getWriter().println( "<A HREF=/viewfile/?filter=StartDownloading>StartDownloading</A><br>" ); 
	                
	        //response.getWriter().append( "hello " + request.getUserPrincipal().getName() + " : " + request.getAuthType() + "<br>" );
	        		
	        //
			// Create a really basic table, showing information about current sessions
			//

            //response.getWriter().println("<table BORDER=1 CELLPADDING=3 CELLSPACING=1 WIDTH=20% >");
			response.getWriter().println("<table BORDER=1 CELLPADDING=3 CELLSPACING=1 >");

			response.getWriter().println( "<td>App</td>");
			response.getWriter().println( "<td>Version</td>");
			response.getWriter().println( "<td>CL</td>");
			response.getWriter().println( "<td>Session</td>");
			response.getWriter().println( "<td>Name</td>");
			response.getWriter().println( "<td>Date</td>");
			response.getWriter().println( "<td>Viewers</td>");
			response.getWriter().println( "<td>Size</td>");
			response.getWriter().println( "<td>Length</td>");
			response.getWriter().println( "<td>Status</td>");
			response.getWriter().println( "<td></td>");

			// Loop over all of the sessions and fill the table
			final List< ReplaySessionInfo > sessions = ReplayDB.discoverSessions( null, 0, 0, null, null, refresh != null ? 10 : 0 );

			response.getWriter().println( "<p>Total Sessions: " + ReplayDB.getNumSessions( null, 0, 0 ) );

			for ( final ReplaySessionInfo sessionEntry : sessions )
	        {
				response.getWriter().println( "<tr>" );

				String LiveString = sessionEntry.bIsIncomplete ? "PARTIAL" : sessionEntry.bIsLive ? "LIVE" : "SAVED";
	       
				DateFormat dateFormat = new SimpleDateFormat("MM/dd/yyyy hh:mm:ss a");
				
				String DateString = dateFormat.format( sessionEntry.createDate );
							
				final int demoTimeInSeconds = sessionEntry.demoTimeInMS / 1000;

				String demoTimeString = String.format( "%d:%02d:%02d", demoTimeInSeconds / 3600, ( demoTimeInSeconds % 3600  ) / 60, ( demoTimeInSeconds % 60 ) );

				final String SizeString = sessionEntry.sizeInBytes > 1024 * 1024 ? String.format( "%2.2f MB", (float)sessionEntry.sizeInBytes / ( 1024 * 1024 ) )  : String.format( "%d KB", sessionEntry.sizeInBytes / ( 1024 ) );  
						
				response.getWriter().println( "<td>" + sessionEntry.appName + "</td>" );
				response.getWriter().println( "<td>" + (long)(sessionEntry.version & 0x00000000ffffffffL ) + "</td>" );
				response.getWriter().println( "<td>" + (long)(sessionEntry.changelist & 0x00000000ffffffffL ) + "</td>" );
				response.getWriter().println( "<td>" + sessionEntry.sessionName + "</td>" );
				response.getWriter().println( "<td>" + sessionEntry.friendlyName + "</td>" );
				response.getWriter().println( "<td>" + DateString + "</td>" );
				response.getWriter().println( "<td>" + sessionEntry.numViewers + "</td>" );
				response.getWriter().println( "<td>" + SizeString + "</td>" );
				response.getWriter().println( "<td>" + demoTimeString + "</td>" );
				response.getWriter().println( "<td>" + LiveString + "</td>" );

				response.getWriter().println( "<td>" );
				
            	if ( !sessionEntry.bIsLive )
            	{
    				final String attributes = "?Session=" + sessionEntry.sessionName;
                    response.getWriter().println( "<A HREF=/deletesession/" + attributes + ">DELETE</A><br>" ); 
            	}

            	response.getWriter().println( "</td>" );

                response.getWriter().println( "</tr>" );                
	        }

			response.getWriter().println( "</table>" );

			final long endTime = System.currentTimeMillis();

			final long totalTime = endTime - startTime;

			response.getWriter().println( "<p>Request took " + totalTime + " ms</p>" );
		}
		catch ( Exception e )
		{
			ReplayLogger.log( "Index. Exception: " + e.getMessage() );
    		e.printStackTrace();
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
		}
    }
}
