/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.InputStream;
import java.io.OutputStream;
import java.io.PrintWriter;
import java.net.UnknownHostException;
import java.text.SimpleDateFormat;
import java.util.ArrayList;
//import java.util.Collection;
import java.util.Date;
import java.util.List;
import java.util.UUID;
import java.util.logging.Level;

import org.bson.types.ObjectId;

import com.mongodb.BasicDBObject;
import com.mongodb.CommandResult;
import com.mongodb.DB;
import com.mongodb.DBCollection;
import com.mongodb.DBCursor;
import com.mongodb.DBObject;
import com.mongodb.MongoClient;
import com.mongodb.WriteConcern;
import com.mongodb.WriteResult;
import com.mongodb.gridfs.GridFS;
import com.mongodb.gridfs.GridFSDBFile;
import com.mongodb.gridfs.GridFSInputFile;

public class MongoDB implements BaseDB 
{
	static MongoClient 	mongoClient = null;
	static DB 			db 			= null;
	static GridFS 		grid 		= null;
	static DBCollection replayColl	= null;
	static DBCollection viewerColl	= null;
	static DBCollection eventColl	= null;
	static DBCollection recentColl	= null;
	static DBCollection logColl		= null;

	public MongoDB() throws UnknownHostException
	{
		mongoClient = new MongoClient( ReplayProps.getString( "mongoDB_Host", "localhost" ) , ReplayProps.getInt( "mongoDB_Port", "27017" ) );

		final boolean bResetDB 		= false;
		final boolean bResetIndexes = ReplayProps.getInt( "mongoDB_ResetIndexes", "0" ) == 1 ? true : false;

		if ( bResetDB )
		{
			mongoClient.dropDatabase( ReplayProps.getString( "mongoDB_DBName", "replayDB" ) );
		}

		db 			= mongoClient.getDB( ReplayProps.getString( "mongoDB_DBName", "replayDB" ) );
		grid 		= new GridFS( db, ReplayProps.getString( "mongoDB_GridFSBucket", "replayFS" ) );
		replayColl 	= db.getCollection( ReplayProps.getString( "mongoDB_ReplayCollection", "replayCollection" ) );
		viewerColl 	= db.getCollection( ReplayProps.getString( "mongoDB_ViewerCollection", "viewerCollection" ) );
		eventColl 	= db.getCollection( ReplayProps.getString( "mongoDB_EventCollection", "eventCollection" ) );
		recentColl 	= db.getCollection( ReplayProps.getString( "mongoDB_RecentCollection", "recentCollection" ) );
		logColl 	= db.getCollection( ReplayProps.getString( "mongoDB_LogCollection", "logCollection" ) );
				
		if ( bResetIndexes )
		{
			ResetIndexes();
		}
	
		RegisterIndexes();

		/*
		listDocuments( replayColl );
		listDocuments( eventColl );
		listDocuments( viewerColl );
		listFiles();
		*/
		
		PrintCollectionStats( replayColl );
		PrintCollectionStats( viewerColl );
		PrintCollectionStats( eventColl );
		PrintCollectionStats( logColl );
		PrintCollectionStats( db.getCollection( "replayFS.files" ) );
		PrintCollectionStats( db.getCollection( "replayFS.chunks" ) );
		
		if ( ReplayProps.getInt( "mongoDB_ConvertReplaysToLower", "0" ) == 1 )
		{
			convertReplaysToLowercase();
		}
	}
	
	public void ResetIndexes()
	{
		replayColl.dropIndex( "*" );
		viewerColl.dropIndex( "*" );
		eventColl.dropIndex( "*" );
		recentColl.dropIndex( "*" );
		logColl.dropIndex( "*" );		
	}

	public void RegisterIndexes()
	{
		EnsureIndex( replayColl, new BasicDBObject( "session", 1 ), "SessionIndex" );
		EnsureIndex( replayColl, new BasicDBObject( "app", 1 ).append( "version", 1 ).append( "bIsLive", 1 ).append( "created", 1 ), "VersionAppSortIndex" );
		EnsureIndex( replayColl, new BasicDBObject( "app", 1 ).append( "bIsLive", 1 ).append( "created", 1 ), "AppSortIndex" );
		EnsureIndex( replayColl, new BasicDBObject( "app", 1 ).append( "meta", 1 ), "AppMetaSortIndex" );
		EnsureIndex( replayColl, new BasicDBObject( "app", 1 ).append( "users", 1 ).append( "created", 1 ), "AppUserIndex" );
		EnsureIndex( replayColl, new BasicDBObject( "created", 1 ), "CreatedIndex" );
		EnsureIndex( replayColl, new BasicDBObject( "modified", 1 ), "ModifiedIndex" );

		EnsureIndex( viewerColl, new BasicDBObject( "session", 1 ), "VSessionIndex" );
		EnsureIndex( viewerColl, new BasicDBObject( "viewer", 1 ), "ViewerIndex" );
		EnsureIndex( viewerColl, new BasicDBObject( "modified", 1 ), "VModifiedIndex" );

		EnsureIndex( recentColl, new BasicDBObject( "user", 1 ), "RecentUserIndex" );

		EnsureIndex( eventColl, new BasicDBObject( "session", 1 ).append( "group", 1 ).append( "time1", 1 ), "ESessionIndex" );
		EnsureIndex( eventColl, new BasicDBObject( "modified", 1 ), "EModifiedIndex" );

		EnsureIndex( logColl, new BasicDBObject( "level", 1 ).append( "created", 1 ), "LevelIndex" );
		EnsureIndex( logColl, new BasicDBObject( "created", 1 ), "LCreatedIndex" );
	}
	
	public void EnsureIndex( DBCollection collection, DBObject keys, String name )
	{
		try
		{
			collection.createIndex( keys, name );
		}
		catch( Exception e )
		{
			replayColl.dropIndex( name );
			collection.createIndex( keys, name );
		}
	}

	public void shutdown()
	{
		mongoClient.close();
	}
	
	public void PrintCollectionStats( final DBCollection coll )
	{
		CommandResult result = coll.getStats();
		
		final String namespace 		= (String)result.get( "ns" );
		final long count 			= ((Number)result.get( "count" )).longValue();
		final long size 			= ((Number)result.get( "size" )).longValue();
		final long storageSize 		= ((Number)result.get( "storageSize" )).longValue();
		final long numIndexes		= ((Number)result.get( "nindexes" )).longValue();
		final long totalIndexSize	= ((Number)result.get( "totalIndexSize" )).longValue();

		// Use our internal logging functions, becauase of the order we are setup and when this function is called, the global logging system isn't initialized yet
		// FIXME: We can fix this by moving the calling of this function to after logging is fully setup
		// For now, this works though
		log( Level.INFO, "Namespace: " + namespace +  ", Count: " + count + ", Size: " + size + ", StorageSize: " + storageSize + ", NumIndexes: " + numIndexes + ", TotalIndexSize: " + totalIndexSize );
	}
	
	// Helpers to build common queries
	static BasicDBObject getSessionQuery( final String sessionName ) 
	{
 		return new BasicDBObject( "session", UUID.fromString( sessionName ) );
 	}

	static BasicDBObject getSessionViewersQuery( final String sessionName ) 
	{
 		return new BasicDBObject( "session", UUID.fromString( sessionName ) );
 	}
 	
	static BasicDBObject getSpecificViewerQuery( final String sessionName, final String viewerName ) 
	{
 		return new BasicDBObject( "session", UUID.fromString( sessionName ) ).append( "viewer", UUID.fromString( viewerName ) );
 	}
 	
	static BasicDBObject getEventQuery( final String sessionName ) 
	{
 		return new BasicDBObject( "session", UUID.fromString( sessionName ) );
 	}

	static String getGridFSFilename( final String sessionName, final String filename )
 	{
		return sessionName + "/" + filename;
 	}

	// UUID helpers
	static String getUUIDString( final DBObject doc, final String key, final boolean bSafe )
	{
		UUID uuid = (UUID)doc.get( key );
		
		if ( uuid == null )
		{
			return bSafe ? "" : null;
		}
		
		return uuid.toString();	
	}

	static String getSessionString( final DBObject doc )
	{
		return getUUIDString( doc, "session", false );
	}

	static String getSessionStringSafe( final DBObject doc )
	{
		return getUUIDString( doc, "session", true );
	}

	static String getViewerString( final DBObject doc )
	{
		return getUUIDString( doc, "viewer", false );
	}

	static String getViewerStringSafe( final DBObject doc )
	{
		return getUUIDString( doc, "viewer", true );
	}

	static void listDocuments( DBCollection coll )
	{
		ReplayLogger.log( "Documents: " + coll );
		DBCursor cursor = coll.find();
		
		while ( cursor.hasNext() ) 
		{
		    DBObject doc = (DBObject)cursor.next();	    
		    ReplayLogger.log( doc.toString() );
		}		
	}
	
	void convertReplaysToLowercase()
	{
		DBCursor cursor = replayColl.find();
		
		while ( cursor.hasNext() ) 
		{
		    DBObject doc = (DBObject)cursor.next();
		    
		    String appName = (String)doc.get( "app" );
		    
		    if ( appName != null )
		    {
		    	appName = appName.toLowerCase();
		    	
		    	doc.put( "app", appName );
		    	
		    	replayColl.save( doc );
		    }
		}		
	}

	static void listFiles()
	{
		ReplayLogger.log( "Files:" );
		DBCursor cursor = grid.getFileList();
		while ( cursor.hasNext() ) 
		{
		    DBObject doc = (DBObject)cursor.next();	    
		    ReplayLogger.log( doc.toString() );
		}
	}

	public boolean sessionExists( final String sessionName )
	{
		return replayColl.count( getSessionQuery( sessionName ) ) > 0;
	}
	
	public void createSession( final String appName, final int version, final int changelist, final String sessionName, final String friendlyName, final List< String > userNames, final String metaString ) throws ReplayException
	{
		if ( sessionExists( sessionName ) )
		{
			throw new ReplayException( "MongoDB.CreateSession: Session already exists." );
		}

		try
		{
			final BasicDBObject newSession = getSessionQuery( sessionName )
							.append( "app", 			appName.toLowerCase() )
							.append( "version", 		version )
							.append( "cl", 				changelist )
							.append( "friendlyName", 	friendlyName )
							.append( "modified", 		System.currentTimeMillis() )
							.append( "created", 		System.currentTimeMillis() )
							.append( "bIsLive", 		(boolean)true )
							.append( "bIncomplete", 	(boolean)false )
							.append( "numChunks", 		(int)0 )
							.append( "demoTimeInMS", 	(int)0 )
							.append( "sizeInBytes", 	(int)0 )
							.append( "users", 			userNames )
							.append( "meta", 			metaString );

		
			replayColl.insert( WriteConcern.ACKNOWLEDGED, newSession );
		}
		catch ( Exception e )
		{
			throw new ReplayException( "MongoDB.CreateSession: There was an error inserting the session" );
		}
	}
	
	public void deleteSession( final String sessionName ) throws ReplayException
	{
		boolean bWasError = false;

		//
		// Remove all items from the DB related to this session
		// If something fails, just keep going
		// This will lessen the chance of orphaned data
		//

		try
		{
			// Remove all documents with matching sessionName
			replayColl.remove( getSessionQuery( sessionName ) );
		}
		catch ( Exception e )
		{
			bWasError = true;	
			ReplayLogger.log( Level.SEVERE, "MongoDB.DeleteSession: replayColl.remove failed. " );
		}
			
		try
		{
			// Remove all matching viewers
			viewerColl.remove( getSessionViewersQuery( sessionName ) );
		}
		catch ( Exception e )
		{
			bWasError = true;	
			ReplayLogger.log( Level.SEVERE, "MongoDB.DeleteSession: viewerColl.remove failed." );
		}
			
		try
		{
			// Remove all matching events
			eventColl.remove( getEventQuery( sessionName ) );
		}
		catch ( Exception e )
		{
			bWasError = true;	
			ReplayLogger.log( Level.SEVERE, "MongoDB.DeleteSession: eventColl.remove failed." );
		}

		try
		{
			// Remove all files that were associated with this session
			grid.remove( new BasicDBObject( "metadata.session", sessionName ) );
		}
		catch ( Exception e )
		{
			bWasError = true;	
			ReplayLogger.log( Level.SEVERE, "MongoDB.DeleteSession: grid.remove failed." );
		}

		if ( bWasError )
		{
			throw new ReplayException( "MongoDB.DeleteSession: WasError == true: " + sessionName );
		}
	}

	public void refreshSession( final String 	sessionName,
								final boolean 	bIsLive, 
								final boolean 	bIncomplete, 
								final int 		numChunks, 
								final int 		demoTimeInMS, 
								final int 		addSizeInBytes ) throws ReplayException
	{
		final BasicDBObject setData = new BasicDBObject()
				.append( "bIsLive", 		bIsLive )
				.append( "bIncomplete", 	bIncomplete )
				.append( "numChunks", 		numChunks )
				.append( "demoTimeInMS", 	demoTimeInMS )
				.append( "modified", 		System.currentTimeMillis() );
		
		final BasicDBObject incData = new BasicDBObject( "sizeInBytes", addSizeInBytes ); 

		WriteResult result = replayColl.update( getSessionQuery( sessionName ), new BasicDBObject( "$set", setData ).append( "$inc", incData ), true, false );
        if ( result.getN() != 1 )
        {
        	throw new ReplayException( "MongoDB.refreshSession: result.getN() != 1" );
        }
	}

	public void refreshSession( final String sessionName, final int addSizeInBytes ) throws ReplayException
	{
		final BasicDBObject setData = new BasicDBObject( "modified", 	System.currentTimeMillis() );
		final BasicDBObject incData = new BasicDBObject( "sizeInBytes", addSizeInBytes ); 

		WriteResult result = replayColl.update( getSessionQuery( sessionName ), new BasicDBObject( "$set", setData ).append( "$inc", incData ), true, false );
		if ( result.getN() != 1 )
		{
			throw new ReplayException( "MongoDB.refreshSession: result.getN() != 1" );
		}
	}

	public void addUsersToSession( final String sessionName, final List< String > userNames ) throws ReplayException
	{
		final BasicDBObject setData 		= new BasicDBObject( "modified", System.currentTimeMillis() );
		final BasicDBObject addToSetData 	= new BasicDBObject( "$each", userNames ); 

		WriteResult result = replayColl.update( getSessionQuery( sessionName ), new BasicDBObject( "$set", setData ).append( "$addToSet", addToSetData ), true, false );

		if ( result.getN() != 1 )
		{
			throw new ReplayException( "MongoDB.addUserToSession: result.getN() != 1" );
		}
	}

	public boolean sessionStreamExists( final String sessionName, final String name )
	{
		try
		{
			return grid.findOne( getGridFSFilename( sessionName, name ) ) != null;
		}
		catch ( Exception e )
		{
			return false;
		}
	}

	public void writeSessionStream( final String sessionName, final String name, final String metaTime1, final String metaTime2, final InputStream inStream ) throws ReplayException
	{
		try
		{
			final String filename = getGridFSFilename( sessionName, name );

			final List< GridFSDBFile > files = grid.find( filename );

			GridFSInputFile file = grid.createFile( inStream );
			file.setChunkSize( 1024 * 64 );
			file.setFilename( filename );
			
			if ( metaTime1 != null && metaTime2 != null )
			{
	    		final int metaTime1Int = Integer.parseInt( metaTime1 );
	    		final int metaTime2Int = Integer.parseInt( metaTime2 );

	    		file.setMetaData( new BasicDBObject( "session", sessionName ).append( "mtime1", metaTime1Int ).append( "mtime2", metaTime2Int ) );
			}
			else
			{
				file.setMetaData( new BasicDBObject( "session", sessionName ) );
			}

			file.save();
			
			for ( final GridFSDBFile fileEntry : files )
			{
				grid.remove( fileEntry );
			}
		}
		catch ( Exception e )
		{
			throw new ReplayException( "MongoDB.WriteSessionStream failed. " + e.getMessage() );
		}
	}

	public BaseDB.StreamMetadata getStreamMetadata( final String sessionName, final String name ) throws ReplayException
	{
		try
		{
			final String filename = getGridFSFilename( sessionName, name );
	
			GridFSDBFile file = grid.findOne( filename );
			
			if ( file == null )
			{
				throw new ReplayException( "MongoDB.GetStreamMetadata file not found. " + filename );
			}
			
        	final long mTime1 = file.getMetaData().containsField( "mtime1" ) ? ((Number)file.getMetaData().get( "mtime1" )).longValue() : 0;
        	final long mTime2 = file.getMetaData().containsField( "mtime2" ) ? ((Number)file.getMetaData().get( "mtime2" )).longValue() : 0;
	        	
        	return new StreamMetadata( mTime1, mTime2 );
	    }
	    catch ( Exception e )
	    {
			throw new ReplayException( "MongoDB.GetStreamMetadata failed. " + e.getMessage() );
	    }
	}

	public long readSessionStream( final String sessionName, final String name, final OutputStream outStream ) throws ReplayException
	{
		try
		{
			final String filename = getGridFSFilename( sessionName, name );
	
			GridFSDBFile file = grid.findOne( filename );
			
			if ( file == null )
			{
				throw new ReplayException( "MongoDB.ReadSessionStream file not found. " + filename );
			}
			
	    	int totalBytes = 0;
	    
			try ( InputStream inStream = file.getInputStream() )
	    	{
	        	byte[] buffer = new byte[1024 * 4];
	        	int readBytes = 0;
	
	        	while ( ( readBytes = inStream.read( buffer ) ) != -1 ) 
	      		{
	        		totalBytes += readBytes;
	        		outStream.write( buffer, 0, readBytes );
	      		}

	        	return totalBytes;
	    	}
	    }
	    catch ( Exception e )
	    {
			throw new ReplayException( "MongoDB.ReadSessionStream failed. " + e.getMessage() );
	    }
	}

	public void writeEventData( final String 	sessionName, 
								final String 	eventGroup,
								final int		eventTime1,
								final int		eventTime2,
								final String 	eventMeta,
								byte[] 			eventData ) throws ReplayException
	{   
        try 
        {
    		final DBObject obj = getEventQuery( sessionName )
    										.append( "group", eventGroup )
    										.append( "time1", eventTime1 )
    										.append( "time2", eventTime2 )
    										.append( "data", eventData );
			if ( eventMeta != null )
			{
				obj.put( "meta", eventMeta );
			}

			eventColl.insert( WriteConcern.ACKNOWLEDGED, obj );
        }
        catch ( Exception e )
        {
        	throw new ReplayException( "MongoDB.WriteEventData: There was an error updating the session" );
        }
	}

	public byte[] readEventData( final String eventId ) throws ReplayException
	{   
        try 
        {
    		final DBObject query = new BasicDBObject( "_id", new ObjectId( eventId ) );
            final DBObject result = eventColl.findOne( query );
            
            if ( result == null )
            {
            	throw new ReplayException( "MongoDB.ReadEventData: Result not found" );
            }
            
            return (byte[])result.get( "data" );
        }
        catch ( Exception e )
        {
        	throw new ReplayException( "MongoDB.WriteEventData: There was an error updating the session" );
        }
	}

	public List<BaseDB.EventInfo> enumerateEvents( final String sessionName, final String eventGroup ) throws ReplayException
	{   
		List<BaseDB.EventInfo> events = new ArrayList<BaseDB.EventInfo>();
		
		try 
        {
			final DBObject query 	= getEventQuery( sessionName ).append( "group", eventGroup );
			final DBObject exclude 	= new BasicDBObject( "data", 0 );
			final DBCursor cursor 	= eventColl.find( query, exclude ).sort( new BasicDBObject( "time1", 1 ) );

            if ( cursor == null )
            {
            	return events;
            }

        	while ( cursor.hasNext() ) 
			{
			    DBObject event = (DBObject)cursor.next();
			    
			    events.add( new BaseDB.EventInfo( ( (ObjectId)event.get( "_id" ) ).toHexString(), (String)event.get( "group" ), (String)event.get( "meta" ), ((Number)event.get( "time1" )).longValue(), ((Number)event.get( "time2" )).longValue() ) );
			}

            return events;
        }
        catch ( Exception e )
        {
        	throw new ReplayException( "MongoDB.enumerateEvents: There was an error reading the event" );
        }
	}

	public boolean viewerExists( final String sessionName, final String viewerName )
	{
		return viewerColl.count( getSpecificViewerQuery( sessionName, viewerName ) ) > 0;
	}

	public void createViewer( final String sessionName, final String viewerName, final String userName ) throws ReplayException
	{
		if ( viewerExists( sessionName, viewerName ) )
		{
			throw new ReplayException( "MongoDB.createViewer: Viewer already exists." );        
		}
		
		try
		{
			viewerColl.insert( WriteConcern.ACKNOWLEDGED, getSpecificViewerQuery( sessionName, viewerName ).append( "modified", System.currentTimeMillis() ) );
		}
		catch ( Exception e )
		{
			throw new ReplayException( "MongoDB.createViewer: Insert failed." );        
		}
		
		addToRecentList( sessionName, userName );
	}
	
	public void addToRecentList( final String sessionName, final String userName ) throws ReplayException
	{
		if ( userName == null || userName.equals( "" ) )
		{
			ReplayLogger.log( Level.WARNING, "addToRecentList: Ignoring null name." );
			return;
		}

		try
		{
			final BasicDBObject query = new BasicDBObject( "user", userName );

			// This is probably slow, but is proof of concept feature, need to make faster
			final int count 	= (int)recentColl.count( query ) + 1;
	 		final int maxCount	= 10;
	 		
	 		if ( count > maxCount )
	 		{
	 			final int numToRemove = count - maxCount;

	 			ReplayLogger.log( Level.INFO, "addToRecentList: Removing: " + numToRemove );
	 			
	 			DBCursor cursor = recentColl.find( query );

		 		cursor.sort( new BasicDBObject( "_id", 1 ) );		// Sort oldest first
		 		cursor.limit( numToRemove );
 		
	 			while ( cursor.hasNext() ) 
	 			{
	 				recentColl.remove( cursor.next() );
	 			}
	 		}
	 		
			final BasicDBObject newObj = new BasicDBObject( "user", userName ).append( "session", UUID.fromString( sessionName ) );

			ReplayLogger.log( Level.INFO, "addToRecentList: Adding: " + sessionName + " : " + userName );

			recentColl.update( newObj, newObj, true, false );
		}
		catch ( Exception e )
		{
			throw new ReplayException( "MongoDB.addToRecentList: Insert failed." );        
		}
	}

	public void refreshViewer( final String sessionName, final String viewerName ) throws ReplayException
	{
		if ( !viewerExists( sessionName, viewerName ) )
		{
			throw new ReplayException( "MongoDB.RefreshViewer: Viewer doesn't exist: " + viewerName );
		}

		try
		{
			final DBObject query = getSpecificViewerQuery( sessionName, viewerName );
	        final DBObject update = new BasicDBObject();
	        update.put( "$set", new BasicDBObject( "modified", System.currentTimeMillis() ) );
	        WriteResult result = viewerColl.update( query, update );
	        if ( result.getN() != 1 )
	        {
				throw new ReplayException( "MongoDB.RefreshViewer result.getN() != 1: " + result.getN() );        
	        }
		}
		catch ( Exception e )
		{
			throw new ReplayException( "MongoDB.RefreshViewer failed." );        
		}
	}

	public void deleteViewer( final String sessionName, final String viewerName ) throws ReplayException
	{
		if ( !viewerExists( sessionName, viewerName ) )
		{
			throw new ReplayException( "MongoDB.DeleteViewer: Viewer doesn't exist." );
		}
		try
		{
			viewerColl.remove( getSpecificViewerQuery( sessionName, viewerName ) );
		}
		catch ( Exception e )
		{
			throw new ReplayException( "MongoDB.DeleteViewer: Delete failed." );
		}
	}

	private Object getFieldValueChecked( final DBObject doc, final String name ) throws ReplayException
	{
		final Object obj = doc.get( name );
		
		if ( obj == null )
		{
			throw new ReplayException( "MongoDB.GetFieldValueChecked. Missing: " + name );
		}
		
		return obj;
	}

	/*
	public static <T> List<T> castList( Class<? extends T> clazz, Collection<?> c ) 
	{
	    List<T> r = new ArrayList<T>( c.size() );

	    for ( Object o : c )
	    {
	    	r.add( clazz.cast( o ) );
	    }

	    return r;
	}
	*/

	private ReplaySessionInfo getSessionInfoFromDoc( final DBObject doc ) throws ReplayException
	{
    	/*
		try
    	{
    		List<String> users = castList( String.class, (List<?>)doc.get( "users" ) );

    	    ReplayLogger.log( Level.INFO, "Users: " + users );
    	}
    	catch( Exception e )
    	{
    	    ReplayLogger.log( Level.INFO, "NULL Users ");
    	}
    	*/

	    try
	    {
	    	final String sessionName = getSessionString( doc );

	    	return new ReplaySessionInfo( 	(String)getFieldValueChecked( doc, "app" ),
										((Number)getFieldValueChecked( doc, "version" )).longValue(),
										((Number)getFieldValueChecked( doc, "cl" )).longValue(),
										sessionName, 
										(String)getFieldValueChecked( doc, "friendlyName" ), 
										(boolean)getFieldValueChecked( doc, "bIsLive" ), 
										(boolean)getFieldValueChecked( doc, "bIncomplete" ), 
										(int)((Number)getFieldValueChecked( doc, "numChunks" )).longValue(),
										(int)((Number)getFieldValueChecked( doc, "demoTimeInMS" )).longValue(),
										//new Date( ((ObjectId)GetFieldValueChecked( doc,  "_id" )).getTimestamp() * 1000L ),
										//((ObjectId)GetFieldValueChecked( doc,  "_id" )).getDate(), 
										new Date( ((Number)getFieldValueChecked( doc, "created" )).longValue() ), 
										new Date( ((Number)getFieldValueChecked( doc, "modified" )).longValue() ), 
										(int)((Number)getFieldValueChecked( doc, "sizeInBytes" )).longValue(),
										(int)viewerColl.count( getSessionViewersQuery( sessionName ) ) );
	    }
	    catch ( Exception e )
	    {
			throw new ReplayException( "MongoDB.GetSessionInfoFromDoc. Exception while reading: " + getSessionStringSafe( doc ) );
	    }
	}

	public ReplaySessionInfo getSessionInfo( final String sessionName ) throws ReplayException
	{	
		try
		{
			return getSessionInfoFromDoc( replayColl.findOne( getSessionQuery( sessionName ) ) );
		}
		catch ( Exception e )
		{
			throw new ReplayException( "GetSessionInfo: There was an error: " + e.getMessage() );
		}
	}
	
	public int getNumSessions( final String appName, final int version, final int changelist )
	{
 		if ( version != 0 )
 		{
 			return (int)replayColl.count( new BasicDBObject( "version", version ) );
 		}

 		return (int)replayColl.count();
	}

	public List<ReplaySessionInfo> discoverSessions( final String appName, final int version, final int changelist, final String userString, final String metaString, final int limit )
	{
		List<ReplaySessionInfo> sessions = new ArrayList<ReplaySessionInfo>();
	
 		DBCursor cursor = null;
 		
 		if ( appName != null )
 		{
 			BasicDBObject query = new BasicDBObject( "app", appName.toLowerCase() );
 			
 			if ( version != 0 )
	 		{
	 			query.append( "version", version );	 			
	 		}

 			if ( metaString != null )
 			{	 				
		 		query.append( "meta", metaString );
 			}

 			if ( userString != null )
 			{
 	 			query.append( "users", userString );
 			}
 			
 			cursor = replayColl.find( query );
 		}
 		else
 		{
 			// This should be only for development
 			cursor = replayColl.find();
 		}

 		// Sort on live status first, then by creation date
 		cursor.sort( new BasicDBObject( "bIsLive", -1 ).append( "created", -1 ) );
	
 		if ( limit != 0 )
		{
 			cursor.limit( limit );
		}

		while ( cursor.hasNext() ) 
		{
		    final DBObject doc = (DBObject)cursor.next();

		    try
		    {
		    	sessions.add( getSessionInfoFromDoc( doc ) );
		    }
		    catch ( Exception e )
		    {
				ReplayLogger.log( Level.SEVERE, "MongoDB.discoverSessions. Exception while reading. Deleting: " + getSessionStringSafe( doc ) );
				safeDeleteSessionWithDocument( getSessionStringSafe( doc ), doc );
		    }
		}				 

		return sessions;
	}

	public List<ReplaySessionInfo> getRecentSessions( final String appName, final int version, final int changelist, final String userName, final int limit )
	{
		List<ReplaySessionInfo> sessions = new ArrayList<ReplaySessionInfo>();
		
 		DBCursor cursor = recentColl.find( new BasicDBObject( "user", userName ) );
 		
 		cursor.sort( new BasicDBObject( "_id", -1 ) );
	
 		if ( limit != 0 )
		{
 			cursor.limit( limit );
		}

		while ( cursor.hasNext() ) 
		{
		    final DBObject doc = (DBObject)cursor.next();

		    try
		    {	    	
		    	final String sessionName = getSessionString( doc );
		    	
		    	final DBObject sessionObj = replayColl.findOne( getSessionQuery( sessionName ) );
		    	
		    	if ( sessionObj == null )
		    	{
					ReplayLogger.log( Level.WARNING, "MongoDB.getRecentSessions. Session not found: " + sessionName );
					recentColl.remove( doc );
		    		continue;
		    	}

		    	ReplaySessionInfo sessionInfo = getSessionInfoFromDoc( sessionObj );

		    	// FIXME: Make this part of the insert/search/schema flow
		    	if ( version != 0 && sessionInfo.version != version )
		    	{
		    		continue;
		    	}

		    	if ( appName != null && !sessionInfo.appName.equals( appName ) )
		    	{
		    		continue;
		    	}

		    	sessions.add( sessionInfo );
		    }
		    catch ( Exception e )
		    {
				ReplayLogger.log( Level.SEVERE, "MongoDB.getRecentSessions. Exception while reading. Deleting: " + getSessionStringSafe( doc ) );
				recentColl.remove( doc );
		    }
		}				 

		return sessions;
	}

	public void log( final Level level, final String str )
	{
		BasicDBObject logObj = new BasicDBObject( "level", level.intValue() ).append( "created", System.currentTimeMillis() ).append( "log", str );
		logColl.insert( logObj );
	}

	public int printLog( final PrintWriter writer, final Level level, final String subFilter )
	{
		int size = 0;

		DBCursor cursor = logColl.find( new BasicDBObject( "level", new BasicDBObject( "$gte", level.intValue() ) ) );
 
 		cursor.sort( new BasicDBObject( "created", 1 ) );

		while ( cursor.hasNext() ) 
		{
		    final DBObject doc = (DBObject)cursor.next();

			try
			{
			    final String str = (String)getFieldValueChecked( doc, "log" );
			    
			    if ( subFilter != null && str.indexOf( subFilter ) == -1 )
			    {
			    	continue;
			    }
			    
				final String dateString = new SimpleDateFormat( "MM/dd/yyyy hh:mm:ss a" ).format( ((ObjectId)getFieldValueChecked( doc,  "_id" )).getDate() );
		    
				final String finalStr = dateString + ": " + str + "<br>";

			    writer.println( finalStr );
			    
			    size += finalStr.length();
			}
			catch ( Exception e )
			{
			}
		}
		
		return size;
	}

	void safeDeleteSessionWithDocument( final String sessionName, final DBObject doc )
	{
		if ( sessionName == null )
		{
			ReplayLogger.log( Level.SEVERE, "MongoDB.SafeDeleteSessionWithDocument: null session name, removing directly.");
			replayColl.remove( doc );
			return;
		}

		try
		{
			deleteSession( sessionName );
		}
		catch ( ReplayException e )
		{
			ReplayLogger.log( Level.SEVERE, "MongoDB.SafeDeleteSessionWithDocument: Failed to delete session: " + sessionName );
			replayColl.remove( doc );
		}
	}

	void cleanupSessions( final DBCursor cursor, final boolean bKeepLive )
	{
		final long startTime = System.currentTimeMillis();

		int numDeletedSessions = 0;

		while ( cursor.hasNext() ) 
		{
		    final DBObject doc = (DBObject)cursor.next();
		    
			final String sessionName = getSessionString( doc );
			
			if ( sessionName == null )
			{
				// This should never happen, corrupt DB?
				replayColl.remove( doc );
				ReplayLogger.log( Level.SEVERE, "MongoDB.CleanupSessions: Invalid session." );
				continue;
			}
			
			final String sessionPath = sessionName;

			if ( !bKeepLive )
			{
				safeDeleteSessionWithDocument( sessionName, doc );
				numDeletedSessions++;
			}
			else
			{
				try
				{
					final ReplaySessionInfo sessionInfo = getSessionInfo( sessionName );
					
					if ( sessionInfo.bIsLive )
					{
						ReplayLogger.log( Level.FINEST, "MongoDB.CleanupSessions: Marking incomplete live session: " + sessionPath );
						
						refreshSession( sessionName, false, true, sessionInfo.numChunks, sessionInfo.demoTimeInMS, sessionInfo.sizeInBytes );
					}
				}
				catch ( ReplayException e )
				{
					ReplayLogger.log( "MongoDB.CleanupSessions: Failed to refresh, deleting stale live session: " + sessionPath );
					safeDeleteSessionWithDocument( sessionName, doc );
					numDeletedSessions++;
				}
			}		
		}

		final long endTime = System.currentTimeMillis();
		
		final long totalTime = endTime - startTime;

		final long maxMinutes = 2;

		if ( totalTime > 60 * 1000 * maxMinutes )
		{
			ReplayLogger.log( Level.WARNING, "MongoDB.CleanupSessions: WARN: Cleanup took: " + totalTime / ( 1000 * 60 ) + " minutes." );
		}

		if ( numDeletedSessions > 0 )
		{
			ReplayLogger.log( Level.FINER, "MongoDB.CleanupSessions: Deleted sessions: " + numDeletedSessions );
		}
	}

	void cleanupStaleLiveSessions()
	{
 		// Find all sessions that haven't been refreshed for more than 60 seconds
		final long maxMinutes = ReplayProps.getInt( "mongoDB_cleanupStaleLiveSessions_MaxMinutes", "1" );

		final DBCursor cursor = replayColl.find( new BasicDBObject( "modified", new BasicDBObject( "$lt", System.currentTimeMillis() - 60 * 1000 * maxMinutes ) ) );//.limit( 20 );

		cleanupSessions( cursor, true );
	}

	void cleanupOldSessions()
	{
 		// Delete all sessions older than 30 days
		final long maxDays = ReplayProps.getInt( "mongoDB_cleanupOldSessions_MaxDays", "30" );

		final DBCursor cursor = replayColl.find( new BasicDBObject( "created", new BasicDBObject( "$lt", System.currentTimeMillis() - 60 * 60 * 24 * maxDays * 1000 ) ) );//.limit( 20 );

		cleanupSessions( cursor, false );
	}

	void cleanupStaleViewers()
	{
 		// Find all viewers that haven't been refreshed for more than 60 seconds
		final long maxMinutes = ReplayProps.getInt( "mongoDB_cleanupStaleViewers_MaxMinutes", "1" );

		final DBCursor cursor = viewerColl.find( new BasicDBObject( "modified", new BasicDBObject( "$lt", System.currentTimeMillis() - 60 * 1000 * maxMinutes ) ) );//.limit( 20 );

		while ( cursor.hasNext() ) 
		{
		    final DBObject doc = (DBObject)cursor.next();

			final String sessionName = getSessionString( doc );
			final String viewerName = getViewerString( doc );
			
			if ( sessionName != null && viewerName != null )
			{   
				ReplayLogger.log( Level.FINER, "MongoDB.CleanupOldViewer: Deleting stale viewer. SessionName: " + sessionName );
			}
			else
			{
				ReplayLogger.log( Level.SEVERE, "MongoDB.CleanupOldViewer: Invalid session." );
			}

			viewerColl.remove( doc );
		}
	}
	
	void cleanupLogs()
	{
		final long maxDays = ReplayProps.getInt( "mongoDB_cleanupLogs_MaxDays", "14" );
		logColl.remove( new BasicDBObject( "created", new BasicDBObject( "$lt", System.currentTimeMillis() - 60 * 60 * 24 * maxDays * 1000 ) ) );

		final long maxFinestDays = ReplayProps.getInt( "mongoDB_cleanupLogs_MaxFinestDays", "3" );
		logColl.remove( new BasicDBObject( "level", new BasicDBObject( "$eq", Level.FINEST.intValue() ) ).append( "created", new BasicDBObject( "$lt", System.currentTimeMillis() - 60 * 60 * 24 * maxFinestDays * 1000 ) ) );
	}
	
	public void quickCleanup()
	{
		cleanupStaleLiveSessions();
		cleanupStaleViewers();
	}

	public void longCleanup()
	{
		cleanupOldSessions();
		cleanupLogs();
		// TODO: Cleanup old files/events that don't have associated sessions
		// One possible way would be to grab all sessions that were refreshed within 14 days
		// Then refresh all events of these sessions
		// Then delete non refreshed events
	}
}
