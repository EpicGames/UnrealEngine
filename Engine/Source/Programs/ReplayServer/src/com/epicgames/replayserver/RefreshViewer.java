/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

public class RefreshViewer extends HttpServlet 
{
	private static final long serialVersionUID = 1L;
    
	protected void doPost( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	try 
    	{ 		
    		final String sessionName = request.getParameter( "Session" );
    		final String viewerName = request.getParameter( "Viewer" );

        	if ( sessionName == null || viewerName == null )
    		{
        		ReplayLogger.log( Level.SEVERE, "RefreshViewer. Missing parameters." );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}

        	final String ViewerFinal = request.getParameter( "Final" );

        	if ( ViewerFinal != null )
        	{
        		// Delete viewer
        		ReplayDB.deleteViewer( sessionName, viewerName );
        		ReplayLogger.log( Level.FINER, "RefreshViewer. Deleting: " + sessionName + "/" + viewerName );
        	}
        	else
        	{
        		// Refresh the viewer
        		ReplayDB.refreshViewer( sessionName, viewerName );
        		ReplayLogger.log( Level.FINEST, "RefreshViewer: " + sessionName + "/" + viewerName );
        	}
    	}
    	catch ( Exception e ) 
    	{
    		ReplayLogger.log( Level.SEVERE, "RefreshViewer. Exception: " + e.getMessage() );
    		e.printStackTrace();
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    }
}
