/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.InputStream;
import java.io.OutputStream;
import java.io.PrintWriter;
import java.util.List;
import java.util.UUID;
import java.util.logging.Level;

public class ReplayDB 
{
	static BaseDB baseDB = null;

	static void init() throws Exception
	{
		try
		{
			baseDB = new MongoDB();
			//baseDB = new FileDB();
		}
		catch ( Exception e )
		{
			throw new Exception( "MongoDB failed to initialize" );
		}		

		boolean bTest = false;

		if ( bTest )
		{
			for ( int i = 0; i < 100; i++ )
			{
				final String app = "testApp";
				
				final String sessionName = createSession( app, 0, 0, "friendlyName", null, null );
				
				if ( i % 10 == 0 )
				{
					for ( int t = 0; t < 10; t++ )
					{
						createViewer( sessionName, "fakeUser" );
					}
				}

				/*
				ReplaySession sessionInfo = GetSessionInfo( sessionVersion, sessionName, true );

				ReplayLogger.Log( sessionInfo.FriendlyName );
		
				for ( int t = 0; t < 5; t++ )
				{
					RefreshSession( sessionVersion, sessionName, true, false, 1, 2, 3 );
					//sessionInfo = GetSessionInfo( sessionVersion, sessionName, true );
				}
				for ( int t = 0; t < 5; t++ )
				{
					FileInputStream inStream = new FileInputStream( FileUtils.RootReplayFolder + "ReplayServer.log" );
					WriteSessionStream( sessionVersion, sessionName, "ReplayServer.log", inStream );
				}
				
				for ( int t = 0; t < 10; t++ )
				{
					CreateViewer( sessionVersion, sessionName );
				}

				sessionInfo = GetSessionInfo( sessionVersion, sessionName, true );
				ReplayLogger.Log( "Viewers: " + sessionInfo.Viewers.size() );
				*/
			}
		}
	}
	
	static void shutdown()
	{
		if ( baseDB != null )
		{
			baseDB.shutdown();
		}
		
		baseDB = null;
	}

	static boolean sessionExists( final String sessionName, final boolean bMustBeLive, final boolean bMustBeFinal )
	{
		if ( !baseDB.sessionExists( sessionName ) )
		{
			return false;
		}
		
		if ( !bMustBeLive && !bMustBeFinal )
		{
			return true;
		}
		
		try
		{
			ReplaySessionInfo sessionInfo = getSessionInfo( sessionName );

			if ( bMustBeLive && !sessionInfo.bIsLive )
			{
				return false;
			}
			
			if ( bMustBeFinal && sessionInfo.bIsLive )
			{
				return false;
			}
		}
		catch ( Exception e )
		{
			return false;
		}
		
		return true;
	}

	static String createSession( final String appName, final int version, final int changelist, final String friendlyName, final List< String > userNames, final String metaString ) throws ReplayException
	{
		final UUID sessionId = java.util.UUID.randomUUID();
		final String sessionName = sessionId.toString();

		try
		{				
			baseDB.createSession( appName, version, changelist, sessionName, friendlyName, userNames, metaString );
			return sessionName; 
		}
		catch ( ReplayException e )
		{
	       	throw new ReplayException( "CreateSession. Failed to create session directory. SessionPath: " + sessionName );
		}
	}

	static void deleteSession( final String sessionName ) throws ReplayException
	{
		baseDB.deleteSession( sessionName );
	}
	
	static void refreshSession( final String sessionName, 
			final boolean 	bIsLive, 
			final boolean 	bIsIncomplete, 
			final int 		NumChunks, 
			final int 		DemoTimeInMS, 
			final int 		SizeInBytes ) throws ReplayException
	{
		baseDB.refreshSession( sessionName, bIsLive, bIsIncomplete, NumChunks, DemoTimeInMS, SizeInBytes );
	}

	static void refreshSession( final String sessionName, final int SizeInBytes ) throws ReplayException
	{
		baseDB.refreshSession( sessionName, SizeInBytes );
	}

	static void finishSession( String SessionName, int NumChunks, int DemoTimeInMS ) throws ReplayException
	{
		refreshSession( SessionName, false, false, NumChunks, DemoTimeInMS, 0 );
	}

	static String createViewer( final String sessionName, final String userName ) throws ReplayException
	{
		try
		{				
			final String viewerName = java.util.UUID.randomUUID().toString();
			baseDB.createViewer( sessionName, viewerName, userName );
			return viewerName; 
		}
		catch ( ReplayException e )
		{
	       	throw new ReplayException( "CreateViewer. Failed to create viewer. SessionPath: " + sessionName );
		}
	}

	static void refreshViewer( final String sessionName, final String viewerName ) throws ReplayException
	{
		baseDB.refreshViewer( sessionName, viewerName );
	}

	static void deleteViewer( final String sessionName, final String viewerName ) throws ReplayException
	{
		baseDB.deleteViewer( sessionName, viewerName );
	}

	static boolean sessionStreamExists( final String sessionName, final String name )
	{
		return baseDB.sessionStreamExists( sessionName, name );
	}

	static void writeSessionStream( final String sessionName, final String name, final String metaTime1, final String metaTime2, final InputStream inStream ) throws ReplayException
	{
		baseDB.writeSessionStream( sessionName, name, metaTime1, metaTime2, inStream );
	}

	static BaseDB.StreamMetadata getStreamMetadata( final String sessionName, final String name ) throws ReplayException
	{
		return baseDB.getStreamMetadata( sessionName, name );
	}

	static long readSessionStream( final String sessionName, final String name, final OutputStream outStream ) throws ReplayException
	{
		return baseDB.readSessionStream( sessionName, name , outStream );
	}
		
	static ReplaySessionInfo getSessionInfo( final String sessionName ) throws ReplayException
	{	
		return baseDB.getSessionInfo( sessionName );
	}

	static void writeEventData( final String 	sessionName, 
								final String 	eventGroup,
								final int		eventTime1,
								final int		eventTime2,
								final String 	eventMeta,
								byte[] 			eventData ) throws ReplayException
	{
		baseDB.writeEventData( sessionName, eventGroup, eventTime1, eventTime2, eventMeta, eventData );
	}

	static byte[] readEventData( final String eventId ) throws ReplayException
	{
		return baseDB.readEventData( eventId );
	}

	static List<BaseDB.EventInfo> enumerateEvents( final String sessionName, final String eventGroup ) throws ReplayException
	{
		return baseDB.enumerateEvents( sessionName, eventGroup );
	}

	static int getNumSessions( final String appName, final int version, final int changelist )
	{
		return baseDB.getNumSessions( appName, version, changelist );
	}

	static List<ReplaySessionInfo> discoverSessions(  final String appName, final int version, final int changelist, final String userString, final String metaString, final int limit )
	{
		return baseDB.discoverSessions( appName, version, changelist, userString, metaString, limit );
	}

	static List<ReplaySessionInfo> getRecentSessions( final String appName, final int version, final int changelist, final String userName, final int limit )
	{
		return baseDB.getRecentSessions( appName, version, changelist, userName, limit );
	}
	
	static void log( final Level level, final String str )
	{
		baseDB.log( level, str );
	}

	static int printLog( final PrintWriter writer, final Level level, final String subFilter )
	{
		return baseDB.printLog( writer, level, subFilter );
	}

	static void quickCleanup()
	{
		baseDB.quickCleanup();
	}

	static void longCleanup()
	{
		baseDB.longCleanup();
	}
}
