/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.util.logging.Level;

public class ReplayException extends Exception 
{
	private static final long serialVersionUID = 1L;

	ReplayException( String detailedDesc )
	{
		super( detailedDesc );
		
		ReplayLogger.log( Level.SEVERE, "EXCEPTION: " + detailedDesc );
	}
}
