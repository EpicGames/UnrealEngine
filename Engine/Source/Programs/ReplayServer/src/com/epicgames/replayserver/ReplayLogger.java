/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.util.logging.Level;

import org.eclipse.jetty.util.log.Logger;

public class ReplayLogger implements Logger 
{ 
	static ReplayLogger	replayLogger = new ReplayLogger();
	
	ReplayLogger()
	{	
	}
		
	String fixArgs( String arg0, Object... arg1 )
	{
		try
		{
	        for ( int i = 0; i < arg1.length; i++ )
	        {
	        	int index = arg0.indexOf( "{}" );
	        	int length = arg0.length();
	        	arg0 = arg0.substring( 0, index ) + arg1[i] + arg0.substring( index + 2, length ); 
	        }
		}
		catch ( Exception e )
		{
			
		}
		
		return arg0;
	}

	String fixArgsLong( String arg0, long... arg1 )
	{
		try
		{
	        for ( int i = 0; i < arg1.length; i++ )
	        {
	        	int index = arg0.indexOf( "{}" );
	        	int length = arg0.length();
	        	arg0 = arg0.substring( 0, index ) + arg1[i] + arg0.substring( index + 2, length ); 
	        }
		}
		catch ( Exception e )
		{
			
		}
		
		return arg0;
	}

	public void debug( String str, long l ) 
	{
		//Log( FixArgsLong( str, l ) );
	}

	public void debug( Throwable arg0 ) 
	{
		log( Level.FINEST, "" + arg0 );
	}

	public void debug( String arg0, Object... arg1 ) 
	{
		log( Level.FINEST, fixArgs( arg0, arg1 ) );
	}

	public void debug( String arg0, Throwable arg1 ) 
	{
		log( Level.FINEST, fixArgs( arg0, arg1 ) );
	}

	public Logger getLogger( String arg0 ) 
	{
		return this;
	}

	public String getName() 
	{
		return null;
	}

	public void ignore( Throwable arg0 ) 
	{
	}

	public void info( Throwable arg0 ) 
	{
		log( Level.FINE, "" + arg0 );
	}

	public void info( String arg0, Object... arg1 ) 
	{
		log( Level.FINE, fixArgs( arg0, arg1 ) );
	}

	public void info( String arg0, Throwable arg1 ) 
	{
		log( Level.FINE, fixArgs( arg0, arg1 ) );
	}

	public boolean isDebugEnabled() 
	{
		return false;
	}

	public void setDebugEnabled(boolean arg0) 
	{
	}

	public void warn(Throwable arg0) 
	{
		log( Level.WARNING, "" + arg0 );
	}

	public void warn(String arg0, Object... arg1) 
	{
		log( Level.WARNING, fixArgs( arg0, arg1 ) );
	}

	public void warn(String arg0, Throwable arg1) 
	{
		log( Level.WARNING, fixArgs( arg0, arg1 ) );
	}
	
	void logInternal( Level level, String str )
	{
		try
		{	
			ReplayDB.log( level, str );
			
			if ( level.intValue() >= Level.FINE.intValue() )
			{
				System.out.println( str );
			}
		}
		catch ( Exception e )
		{
			System.out.println( "Log exception: " + e.getMessage() );
		}
	}

	static void log( String str )
	{
		replayLogger.logInternal( Level.FINE, str );
	}

	static void log( Level level, String str )
	{
		replayLogger.logInternal( level, str );
	}
}