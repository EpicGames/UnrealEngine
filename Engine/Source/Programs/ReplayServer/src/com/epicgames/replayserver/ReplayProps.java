package com.epicgames.replayserver;

import java.io.File;
import java.io.FileInputStream;
import java.io.FileOutputStream;
import java.io.IOException;
import java.io.InputStream;
import java.util.Properties;

public class ReplayProps 
{
	static Properties props = new Properties();
	
	ReplayProps()
	{
	}
	
	static void Init() throws IOException
	{
		try ( final InputStream is = new FileInputStream( new File( "ReplayServer.Properties" ) ) )
		{
			props.load( is );
		}
	}
	
	static String getString( final String key, final String defaultValue )
	{
		return props.getProperty( key, defaultValue );
	}
	
	static int getInt( final String key, final String defaultValue )
	{
		return Integer.parseInt( getString( key, defaultValue ) );
	}
	
	static void SaveProps()
	{
		try ( final FileOutputStream out = new FileOutputStream( new File( "ReplayServer.Properties" ) ) ) 
		{
			props.store( out, "ReplayServer Properties" );
		}
		catch ( Exception e )
		{
		}		
	}
}
