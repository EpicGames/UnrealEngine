/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.util.Timer;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.ServletInputStream;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

import org.eclipse.jetty.security.ConstraintMapping;
import org.eclipse.jetty.security.ConstraintSecurityHandler;
import org.eclipse.jetty.security.HashLoginService;
import org.eclipse.jetty.security.authentication.FormAuthenticator;
import org.eclipse.jetty.server.Server;
import org.eclipse.jetty.servlet.*;
import org.eclipse.jetty.util.security.Constraint;
import org.eclipse.jetty.util.security.Password;

public class ReplayServer 
{
	static Timer timer = null;
	
	public static void main( String[] args ) throws Exception
    {
		ReplayProps.Init();

		ReplayDB.init();

		ReplayLogger.log( Level.INFO, "Log opened..." );

		org.eclipse.jetty.util.log.Log.setLog( ReplayLogger.replayLogger );

		final int quickTaskFrequencyInSeconds 	= ReplayProps.getInt( "quickTaskFrequencyInSeconds", "10" );
		final int longTaskFrequencyInHours 		= ReplayProps.getInt( "longTaskFrequencyInHours", "1" );
	
		timer = new Timer();

		if ( quickTaskFrequencyInSeconds != 0 )
		{
			timer.schedule( new CleanupTask.QuickTask(), quickTaskFrequencyInSeconds * 1000, quickTaskFrequencyInSeconds * 1000 );		
		}

		if ( quickTaskFrequencyInSeconds != 0 )
		{
			timer.schedule( new CleanupTask.LongTask(), 60 * 60 * 1000 * longTaskFrequencyInHours, 60 * 60 * 1000 * longTaskFrequencyInHours );		
		}
		
		final int httpPort = ReplayProps.getInt( "httpPort", "80" );

		final Server server = new Server( httpPort );
 
        ServletContextHandler context = new ServletContextHandler( ServletContextHandler.SESSIONS | ServletContextHandler.SECURITY );
        context.setContextPath( "/" );
        server.setHandler( context );

        // Change default timeout to 45 seconds (from 30)
        /*
        ServerConnector http = new ServerConnector( server );
        //http.setHost( "localhost" );
        http.setPort( httpPort );
        http.setIdleTimeout( 45000 );

        server.addConnector( http );
        */

        final boolean bTestAuthorize = false;

        if ( bTestAuthorize )
        {
            context.addServlet(new ServletHolder(new HttpServlet() 
            {
            	private static final long serialVersionUID = 1L;
            	@Override
            	protected void doPost(HttpServletRequest request, HttpServletResponse response) throws ServletException, IOException 
            	{
            		final ServletInputStream inputStream = request.getInputStream();
                    		
            		FileUtils.ReplayOutputStream outputStream = FileUtils.getReplayOutputStream( "authorize.txt", false );

                    byte[] buffer = new byte[1024];

                	int readBytes = 0;

                	while ( ( readBytes = inputStream.read( buffer ) ) != -1 ) 
              		{
              			outputStream.write(  buffer, 0, readBytes );
              		}

                	inputStream.close();
                	outputStream.close();

                	response.setContentType( "text/html" );
                    response.getWriter().println( "<p>Worked: " + request.getContentType() + " </p><br>" );
            	
            	}
            }), "/j_security_check" );
        }

        context.addServlet( new ServletHolder( new Download() ), "/download/*" );
        context.addServlet( new ServletHolder( new DownloadEvent() ), "/downloadevent/*" );
        context.addServlet( new ServletHolder( new Upload() ), "/upload/*" );
        context.addServlet( new ServletHolder( new UploadEvent() ), "/uploadevent/*" );
        context.addServlet( new ServletHolder( new StartUploading() ), "/startuploading/*" );
        context.addServlet( new ServletHolder( new StopUploading() ), "/stopuploading/*" );
        context.addServlet( new ServletHolder( new StartDownloading() ), "/startdownloading/*" );
        context.addServlet( new ServletHolder( new DeleteSession() ), "/deletesession/*" );
        context.addServlet( new ServletHolder( new EnumerateSessions() ), "/enumsessions/*" );
        context.addServlet( new ServletHolder( new EnumEvents() ), "/enumevents/*" );
        context.addServlet( new ServletHolder( new RefreshViewer() ), "/refreshviewer/*" );
        context.addServlet( new ServletHolder( new ViewFile() ), "/viewfile/*" );
        
        if ( ReplayProps.getInt( "enableBuiltInWebServer", "1" ) == 1 )
        {
            context.addServlet( new ServletHolder( new Index() ), "/*" );
        }
        
        // Password protect the delete session page
        context.addServlet(new ServletHolder(new HttpServlet() 
        {
        	private static final long serialVersionUID = 1L;
        	@Override
        	protected void doGet(HttpServletRequest request, HttpServletResponse response) throws ServletException, IOException 
        	{
      			response.setContentType( "text/html" );
      			response.getWriter().append( "<form method='POST' action='/j_security_check'>"
      				+ "<input type='text' name='j_username'/>"
      				+ "<input type='password' name='j_password'/>"
      				+ "<input type='submit' value='Login'/></form>");
        	}
        }), "/login" );
 
        final Constraint constraint = new Constraint();
        constraint.setName( Constraint.__FORM_AUTH );
        constraint.setRoles( new String[]{ "user", "admin", "moderator" } );
        constraint.setAuthenticate( true );

        final ConstraintMapping constraintMapping = new ConstraintMapping();
        constraintMapping.setConstraint( constraint );
        constraintMapping.setPathSpec( "/deletesession/*" );

        final ConstraintSecurityHandler securityHandler = new ConstraintSecurityHandler();
        securityHandler.addConstraintMapping( constraintMapping );
        HashLoginService loginService = new HashLoginService();
        loginService.putUser( "usern", new Password( "pass" ), new String[] { "user" } );
        loginService.putUser( "admin", new Password( "pass" ), new String[] { "admin" } );
        securityHandler.setLoginService( loginService );

        final FormAuthenticator authenticator = new FormAuthenticator( "/login", "/login", false );
        securityHandler.setAuthenticator( authenticator );

        if ( !bTestAuthorize )
        {
            context.setSecurityHandler( securityHandler );
        }

        server.start();
        server.join();
        
        ReplayDB.shutdown();
    }
}
