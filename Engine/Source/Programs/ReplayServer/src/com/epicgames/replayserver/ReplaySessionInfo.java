/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.util.Date;

public class ReplaySessionInfo 
{
	String 		appName			= null;
	long		version			= 0;
	long		changelist		= 0;
	boolean 	bIsLive			= false;
	boolean 	bIsIncomplete	= false;
	String 		sessionName		= null;
	String 		friendlyName	= null;
	int			numChunks		= 0;
	int 		demoTimeInMS	= 0;
	Date 		createDate		= null;
	Date 		refreshDate		= null;
	int			numViewers		= 0;
	int			sizeInBytes		= 0;
	
	ReplaySessionInfo( String inAppName, long inNetVersion, long inChanglelist, String inSessionName, String inFriendlyName, boolean inbIsLive, boolean bInIsIncomplete, int inNumChunks, int inDemoTimeInMS, Date inCreateDate, Date inRefreshDate, int inSizeInBytes, int inNumViewers )
	{
		appName			= inAppName;
		version			= inNetVersion;
		changelist		= inChanglelist;
		sessionName 	= inSessionName;
		friendlyName 	= inFriendlyName;
		bIsLive 		= inbIsLive;
		bIsIncomplete	= bInIsIncomplete;
		numChunks 		= inNumChunks;
		demoTimeInMS	= inDemoTimeInMS;
		createDate		= inCreateDate;
		refreshDate		= inRefreshDate;
		sizeInBytes		= inSizeInBytes;
		numViewers		= inNumViewers;
	}
	
	public long GetRefreshTimeInSeconds()
	{
		return ( System.currentTimeMillis() - refreshDate.getTime() ) / 1000;  
	}
}
