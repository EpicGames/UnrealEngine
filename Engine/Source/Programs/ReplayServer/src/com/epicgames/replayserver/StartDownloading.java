/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;


public class StartDownloading extends HttpServlet 
{
	private static final long serialVersionUID = 1L;

	public StartDownloading()
	{		
	}

    protected void doPost( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	try
    	{
    		final String sessionName = request.getParameter( "Session" );
    		final String userName = request.getParameter( "User" );

            if ( !ReplayDB.sessionExists( sessionName, false, false ) )
        	{
            	ReplayLogger.log( Level.SEVERE, "StartDownloading. Session doesn't exist: " + sessionName );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
        		return;
        	}

            final ReplaySessionInfo session = ReplayDB.getSessionInfo( sessionName );
            
            if ( session.bIsLive )
            {
        		response.setHeader( "State", "Live" );
            }
            else
            {
                response.setHeader( "State", "Final" );
            }

            final String viewerName = ReplayDB.createViewer( sessionName, userName );
    		
    		response.setHeader( "NumChunks", "" + session.numChunks );
    		response.setHeader( "Time", "" + session.demoTimeInMS );
    		response.setHeader( "Viewer", viewerName );
 
    		ReplayLogger.log( Level.FINE, "StartDownloading. Path: " + session.appName + "/" + session.version + "/" + session.changelist + "/" + sessionName + " NumChunks: " + session.numChunks + " DemoTimeInMS: " + session.demoTimeInMS + " Viewer: " + viewerName );
    	}
    	catch ( ReplayException e )
    	{
    		ReplayLogger.log( Level.SEVERE, "StartDownloading. ReplayException: " + e.getMessage() );
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    	catch ( Exception e )
    	{
    		ReplayLogger.log( Level.SEVERE, "StartDownloading. Exception: " + e.getMessage() );
    		e.printStackTrace();
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    }
}
