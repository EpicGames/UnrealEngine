/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.io.InputStreamReader;
import java.util.List;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

import com.google.gson.Gson;

public class StartUploading extends HttpServlet 
{
	private static final long serialVersionUID = 1L;

	public class ReplayUsers 
	{
		public List< String > users;
	}

	public StartUploading()
	{		
	}

    protected void doPost( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	try
    	{
        	final String appName 		= request.getParameter( "App" );
        	final String versionString 	= request.getParameter( "Version" );
        	final String clString 		= request.getParameter( "CL" );
    		final String friendlyName 	= request.getParameter( "Friendly" );
    		final String metaString 	= request.getParameter( "Meta" );
    		
    		if ( appName == null )
    		{
        		ReplayLogger.log( Level.SEVERE, "StartUploading. appName == null." );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}
        	
    		if ( versionString == null )
    		{
        		ReplayLogger.log( Level.SEVERE, "StartUploading. versionString == null." );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}

    		if ( clString == null )
    		{
        		ReplayLogger.log( Level.SEVERE, "StartUploading. clString == null." );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}

    		if ( friendlyName == null )
    		{
        		ReplayLogger.log( Level.SEVERE, "StartUploading. FriendlyName == null." );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}

    		final int version = Integer.parseUnsignedInt( versionString );
        	final int changelist = Integer.parseUnsignedInt( clString );

        	List< String > userNames = null;
       
        	if ( request.getContentLength() > 0 )
        	{
            	try ( InputStreamReader inputStreamReader = new InputStreamReader( request.getInputStream() ) )
        		{
            		ReplayUsers replayUsers = new Gson().fromJson( inputStreamReader, ReplayUsers.class );
            		userNames = replayUsers.users;
        		}
            	catch( Exception e )
            	{
            		ReplayLogger.log( Level.SEVERE, "StartUploading: Stream failed: " + appName + "/" + version );
                    response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                    return;
            	}
        	}

            // Start a live session
    		final String SessionName = ReplayDB.createSession( appName, version, changelist, friendlyName, userNames, metaString );
      	
    		ReplayLogger.log( Level.FINE, "StartUploading. Success: " + appName + "/" + versionString + "/" + clString + "/" + SessionName + "/" + friendlyName + "/" + metaString );

    		response.setHeader( "Session",  SessionName );   		
    	}
    	catch ( ReplayException e )
    	{
    		ReplayLogger.log( Level.SEVERE, "StartUploading. ReplayException: " + e.getMessage() );
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    	catch ( Exception e )
    	{
    		ReplayLogger.log( Level.SEVERE, "StartUploading. Exception: " + e.getMessage() );
    		e.printStackTrace();
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    }
}
