/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

public class StopUploading extends HttpServlet
{
	private static final long serialVersionUID = 1L;

	public StopUploading()
	{		
	}

    protected void doPost( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	try
    	{
        	final String sessionName = request.getParameter( "Session" );
        	final String numChunksString = request.getParameter( "NumChunks" );
        	final String demoTimeString = request.getParameter( "Time" );
    		
    		if ( sessionName == null || numChunksString == null || demoTimeString == null )
    		{
    			ReplayLogger.log( Level.SEVERE, "StopUploading. Missing parameters." );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}

    		final int numChunks = Integer.parseInt( numChunksString );
    		final int demoTimeInMS = Integer.parseInt( demoTimeString );
        	
			ReplayDB.finishSession( sessionName, numChunks, demoTimeInMS );

			ReplayLogger.log( Level.FINE, "StopUploading. Success: " + sessionName + " NumChunks: " + numChunks + " DemoTimeInMS: " + demoTimeInMS );
    	}
    	catch ( ReplayException e )
    	{
    		ReplayLogger.log( "StopUploading. ReplayException: " + e.getMessage() );
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    	catch ( Exception e )
    	{
    		ReplayLogger.log( "StopUploading. Exception: " + e.getMessage() );
    		e.printStackTrace();
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    }
}
