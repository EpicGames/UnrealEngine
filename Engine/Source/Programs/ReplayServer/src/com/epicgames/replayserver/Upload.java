/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.ServletInputStream;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

public class Upload extends HttpServlet
{
	private static final long serialVersionUID = 1L;

	public Upload()
	{		
	}

	protected void doPost( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	try 
    	{
    		final long StartTime = System.currentTimeMillis();
   		
    		final String sessionName = request.getParameter( "Session" );
    		final String filename = request.getParameter( "Filename" );
        	final String numChunksString = request.getParameter( "NumChunks" );
        	final String demoTimeString = request.getParameter( "Time" );

        	if ( sessionName == null || filename == null || numChunksString == null || demoTimeString == null )
    		{
        		ReplayLogger.log( Level.SEVERE, "Upload. Missing parameters." );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}

    		final int numChunks = Integer.parseInt( numChunksString );
    		final int demoTimeInMS = Integer.parseInt( demoTimeString );

            final String path = sessionName + "/" + filename;

    		if ( !ReplayDB.sessionExists( sessionName, true, false ) )
        	{
            	ReplayLogger.log( Level.SEVERE, "Upload. Session doesn't exist: " + path );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
        		return;
        	}
           
        	final String metaTime1 = request.getParameter( "MTime1" );
        	final String metaTime2 = request.getParameter( "MTime2" );
        	
    		try ( final ServletInputStream serveletStream = request.getInputStream() )
            {
                ReplayDB.writeSessionStream( sessionName, filename, metaTime1, metaTime2, serveletStream );
            }
            catch ( Exception e )
            {
        		ReplayLogger.log( Level.SEVERE, "Upload: Stream failed: " + path );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
            }

        	// Refresh the session
    		ReplayDB.refreshSession( sessionName, true, false, numChunks, demoTimeInMS, request.getContentLength() );

    		final long endTime = System.currentTimeMillis();
    		
    		final long totalTime = endTime - StartTime;

    		if ( totalTime > 5000 )
    		{
                ReplayLogger.log( Level.WARNING, "Upload. Path: " + path + "/" + filename + ", Size: " + request.getContentLength() + " NumChunks: " + numChunks + " DemoTimeInMS: " + demoTimeInMS + " Time: " + totalTime + " ms"  );
    		}
    	}
    	catch ( Exception e ) 
    	{
    		ReplayLogger.log( Level.SEVERE, "Upload. Exception: " + e.getMessage() );
    		e.printStackTrace();
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    }
}
