/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.IOException;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.ServletInputStream;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

public class UploadEvent extends HttpServlet 
{
	private static final long serialVersionUID = 1L;
	
	protected void doPost( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	try 
    	{  		
    		final String sessionName 	= request.getParameter( "Session" );
        	final String eventGroup 	= request.getParameter( "Group" );
        	final String eventTime1 	= request.getParameter( "Time1" );
        	final String eventTime2 	= request.getParameter( "Time2" );
        	final String eventMeta 		= request.getParameter( "Meta" );

        	if ( sessionName == null || eventGroup == null || eventTime1 == null || eventTime2 == null )
    		{
        		ReplayLogger.log( "UploadEvent. Missing parameters." );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
    		}

    		if ( !ReplayDB.sessionExists( sessionName, true, false ) )
        	{
            	ReplayLogger.log( "UploadEvent. Session doesn't exist: " + sessionName );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
        		return;
        	}
                  	
    		final int time1 = Integer.parseInt( eventTime1 );
    		final int time2 = Integer.parseInt( eventTime2 );
   		
    		if ( request.getContentLength() > 1024 * 1024 )
    		{
        		ReplayLogger.log( Level.SEVERE, "UploadEvent: WARN: Large event data. Path: " + sessionName + " Size: " + request.getContentLength() );
    		}

    		try ( final ServletInputStream serveletStream = request.getInputStream() )
            {
    	    	int totalBytes = 0;
    		    
	        	byte[] eventData = new byte[request.getContentLength()];
	        	int readBytes = 0;
	
	        	// Completely read the buffer
	        	while ( ( readBytes = serveletStream.read( eventData, totalBytes, request.getContentLength() ) ) != -1 ) 
	      		{
	        		totalBytes += readBytes;
	      		}
	        	
	        	if ( totalBytes != request.getContentLength() )
	        	{
	        		ReplayLogger.log( Level.SEVERE, "UploadEvent: totalBytes != request.getContentLength(): " + sessionName );
	                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
	                return;
	        	}
	        		        	                
	    		final long StartTime = System.currentTimeMillis();

	    		// Write the data to the db
	        	ReplayDB.writeEventData( sessionName, eventGroup, time1, time2, eventMeta, eventData );

	    		ReplayDB.refreshSession( sessionName, request.getContentLength() );

	    		final long endTime = System.currentTimeMillis();
	    		
	    		final long totalTime = endTime - StartTime;

	    		if ( totalTime > 1000 )
	    		{
	                ReplayLogger.log( Level.WARNING, "UploadEvent. Path: " + sessionName + ", Size: " + request.getContentLength()  + " Time: " + totalTime + " ms"  );
	    		}
	    		else
	    		{
	                ReplayLogger.log( Level.FINEST, "UploadEvent. Path: " + sessionName + ", Size: " + request.getContentLength()  + " Time: " + totalTime + " ms"  );
	    		}
            }
            catch ( Exception e )
            {
        		ReplayLogger.log( "UploadEvent: WriteEventData failed: " + sessionName );
                response.sendError( HttpServletResponse.SC_BAD_REQUEST );
                return;
            }
    	}
    	catch ( Exception e ) 
    	{
    		ReplayLogger.log( "UploadEvent. Exception: " + e.getMessage() );
    		e.printStackTrace();
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    }	
}
