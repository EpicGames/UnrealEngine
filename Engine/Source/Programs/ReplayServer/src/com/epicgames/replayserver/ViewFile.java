/*
 * Copyright (C) Epic Games, Inc. All Rights Reserved.
 */
package com.epicgames.replayserver;

import java.io.FileNotFoundException;
import java.io.IOException;
import java.util.logging.Level;

import javax.servlet.ServletException;
import javax.servlet.http.HttpServlet;
import javax.servlet.http.HttpServletRequest;
import javax.servlet.http.HttpServletResponse;

public class ViewFile extends HttpServlet 
{
	private static final long serialVersionUID = 1L;
	
	protected void doGet( HttpServletRequest request, HttpServletResponse response ) throws ServletException, IOException
    {
    	try 
    	{                  	
    		final long startTime = System.currentTimeMillis();
    		 		
    		final String levelStr = request.getParameter( "level" );
    		final String filterStr = request.getParameter( "filter" );

			response.setContentType( "text/html" );
            response.getWriter().println( "<A HREF=/>GO BACK</A><br>" );

			final Level level = levelStr != null ? Level.parse( levelStr ) : Level.ALL;

			final int size = ReplayDB.printLog( response.getWriter(), level, filterStr );

    		final long endTime = System.currentTimeMillis();
    		
    		final long totalTime = endTime - startTime;

    		final String log = "ViewFile. Size: " + size + " Time: " + totalTime + " ms";

    		response.getWriter().println( "<p>" + log + "</p><br>" );
            response.getWriter().println( "<A HREF=/>GO BACK</A><br>" );

            if ( totalTime > 2000 )
            {
                ReplayLogger.log( Level.WARNING, log );
            }

            response.setStatus( HttpServletResponse.SC_OK );
    	} 
    	catch ( FileNotFoundException e )
    	{
    		ReplayLogger.log( Level.SEVERE, "File not found: " + request.getPathInfo() );
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    	catch ( Exception e ) 
    	{
    		ReplayLogger.log( Level.SEVERE, "Download. Exception: " + e.getMessage() );
    		e.printStackTrace();
            response.sendError( HttpServletResponse.SC_BAD_REQUEST );
    	}
    }	
}
